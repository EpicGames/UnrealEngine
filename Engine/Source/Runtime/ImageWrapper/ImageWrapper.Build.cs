// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ImageWrapper : ModuleRules
{
	protected virtual bool bEnableMinimalEXR { get => false; }

	public ImageWrapper(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PublicIncludePathModuleNames.AddRange(new string[] {
			"ImageCore"
			}
		);

		// include only, no link :
		// for TextureDefines.h :
		PrivateIncludePathModuleNames.AddRange(new string[]
		{
			"Engine",
		});

		PublicDefinitions.Add("WITH_UNREALPNG=1");
		PublicDefinitions.Add("WITH_UNREALJPEG=1");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			}
		);

		PublicDependencyModuleNames.Add("LibTiff");
		PublicDependencyModuleNames.Add("ImageCore");

		AddEngineThirdPartyPrivateStaticDependencies(Target,
			"zlib",
			"UElibPNG",
			"LibTiff",
			"OodleDataCompression",
			"UEJpegComp"
		);

		// Jpeg Decoding
		// LibJpegTurbo is much faster than UElibJPG but has not been compiled or tested for all platforms
		// Note that currently this module is included at runtime, so consider the increase in exe size before
		// enabling for any of the console/phone platforms!

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows)
			|| Target.Platform == UnrealTargetPlatform.Mac
			|| Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
		{
			PublicDefinitions.Add("WITH_LIBJPEGTURBO=1");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "LibJpegTurbo");
		}
		else
		{
			PublicDefinitions.Add("WITH_LIBJPEGTURBO=0");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "UElibJPG");
		}

		bool bEnableEXR = false;
		bool bUseMinimalEXR = bEnableMinimalEXR;

		if (Target.Type != TargetType.Server)
		{
			// Add openEXR lib for desktop builds.
			if ((Target.Platform.IsInGroup(UnrealPlatformGroup.Windows)) ||
				(Target.Platform == UnrealTargetPlatform.Mac) ||
				(Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) && Target.Architecture == UnrealArch.X64))
			{
				bEnableEXR = true;
				bUseMinimalEXR = false;
			}
		}

		if (bEnableEXR)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "Imath");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "UEOpenExr");
		}

		PublicDefinitions.Add("WITH_UNREALEXR=" + (bEnableEXR ? "1" : "0"));
		PublicDefinitions.Add("WITH_UNREALEXR_MINIMAL=" + (bUseMinimalEXR ? "1" : "0"));

		// Enable exceptions to allow error handling
		bEnableExceptions = true;

		bDisableAutoRTFMInstrumentation = true;
	}
}
