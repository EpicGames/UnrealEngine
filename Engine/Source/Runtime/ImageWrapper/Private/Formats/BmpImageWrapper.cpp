// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formats/BmpImageWrapper.h"
#include "ImageWrapperPrivate.h"

#include "BmpImageSupport.h"
#include "ImageCoreUtils.h"

static inline bool BmpDimensionIsValid(int32 Dim)
{
	//ensures we can do Width*4 in int32
	//	and then also call Align() on it
	const int32 BmpMaxDimension = (INT32_MAX/4) - 2;

	// zero dimension could be technically valid, but we won't load it
	return Dim > 0 && Dim <= BmpMaxDimension;
}

/**
 * BMP image wrapper class.
 * This code was adapted from UTextureFactory::ImportTexture, but has not been throughly tested.
 */

FBmpImageWrapper::FBmpImageWrapper(bool bInHasHeader, bool bInHalfHeight)
	: FImageWrapperBase()
	, bHasHeader(bInHasHeader)
	, bHalfHeight(bInHalfHeight)
{
	// IcoImageWrapper uses FBmpImageWrapper(false, true));
}

void FBmpImageWrapper::Uncompress(const ERGBFormat InFormat, const int32 InBitDepth)
{
	RawData.Empty();
	if ( CompressedData.IsEmpty() )
	{
		return;
	}

	const uint8* Buffer = CompressedData.GetData();

	if (!bHasHeader || ((CompressedData.Num()>=sizeof(FBitmapFileHeader)+sizeof(FBitmapInfoHeader)) && Buffer[0]=='B' && Buffer[1]=='M'))
	{
		if ( ! UncompressBMPData(InFormat, InBitDepth) )
		{
			RawData.Empty();
			if ( LastError.IsEmpty() )
			{
				SetError(TEXT("UncompressBMPData failed"));
			}
		}
	}
}

static inline bool SafeAdvancePointer(const uint8 *& OutPtr, const uint8 * StartPtr, const uint8 * EndPtr, ptrdiff_t Step)
{
	if ( Step < 0 || Step > (EndPtr - StartPtr) )
	{
		return false;
	}
	OutPtr = StartPtr + Step;
	return true;
}


bool FBmpImageWrapper::UncompressBMPData(const ERGBFormat InFormat, const int32 InBitDepth)
{
	// always writes BGRA8 :
	check( InFormat == ERGBFormat::BGRA );
	check( InBitDepth == 8 );

	// was checked before calling here :
	check( CompressedData.Num() >= sizeof(FBitmapFileHeader)+sizeof(FBitmapInfoHeader) );

	const uint8* const Buffer = CompressedData.GetData();
	const uint8* const BufferEnd = Buffer + CompressedData.Num();

	const FBitmapInfoHeader* bmhdr = nullptr;
	ptrdiff_t BitsOffset = 0;
	EBitmapHeaderVersion HeaderVersion = EBitmapHeaderVersion::BHV_BITMAPINFOHEADER;

	if (bHasHeader)
	{
		bmhdr = (FBitmapInfoHeader *)(Buffer + sizeof(FBitmapFileHeader));

		const FBitmapFileHeader * bmfh = (const FBitmapFileHeader*) Buffer;
		BitsOffset = bmfh->bfOffBits;
	}
	else
	{
		// used for ICO loading
		bmhdr = (FBitmapInfoHeader *)Buffer;
		BitsOffset = bmhdr->biSize;
	}
	
	HeaderVersion = bmhdr->GetHeaderVersion();
	if ( HeaderVersion == EBitmapHeaderVersion::BHV_INVALID )
	{
		UE_LOGF(LogImageWrapper, Error, "BitmapHeaderVersion invalid");
		return false;
	}

	/*
	UE_LOGF(LogImageWrapper, Log, "BMP compression = (%i) BitCount = (%i)", bmhdr->biCompression,bmhdr->biBitCount)
	UE_LOGF(LogImageWrapper, Log, "BMP BitsOffset = (%i)", BitsOffset)
	UE_LOGF(LogImageWrapper, Log, "BMP biSize = (%i)", bmhdr->biSize)
	*/

	if (bmhdr->biCompression != BCBI_RGB && bmhdr->biCompression != BCBI_BITFIELDS && bmhdr->biCompression != BCBI_ALPHABITFIELDS)
	{
		UE_LOGF(LogImageWrapper, Error, "RLE compression of BMP images not supported");
		return false;
	}
	
	if (bmhdr->biPlanes != 1 )
	{
		UE_LOGF(LogImageWrapper, Error, "BMP uses an unsupported biPlanes != 1");
		return false;
	}
	
	if ( bmhdr->biBitCount != 8 && bmhdr->biBitCount != 16 && bmhdr->biBitCount != 24 && bmhdr->biBitCount != 32 )
	{
		UE_LOGF(LogImageWrapper, Error, "BMP uses an unsupported biBitCount (%i)", bmhdr->biBitCount);
		return false;
	}
	
	const uint8* Bits;
	if ( ! SafeAdvancePointer(Bits,Buffer,BufferEnd,BitsOffset) )
	{
		UE_LOGF(LogImageWrapper, Error, "Bmp read would overrun buffer");
		return false;
	}
	
	const uint8 * AfterHeader;
	if ( ! SafeAdvancePointer(AfterHeader, (const uint8 *)bmhdr, BufferEnd, bmhdr->biSize) )
	{
		UE_LOGF(LogImageWrapper, Error, "Bmp read would overrun buffer");
		return false;
	}

	// Set texture properties.
	// This should have already been set by LoadHeader from SetCompressed

	check( Format == ERGBFormat::BGRA );
	check( BitDepth == 8 );

	Width = bmhdr->biWidth;
	const bool bNegativeHeight = (bmhdr->biHeight < 0);
	Height = FMath::Abs(bmhdr->biHeight);
	if ( bHalfHeight )
	{
		Height /= 2;
	}
	if ( ! BmpDimensionIsValid(Width) || ! BmpDimensionIsValid(Height) )
	{
		UE_LOGF(LogImageWrapper, Error, "Bmp dimensions invalid");
		return false;
	}

	int64 RawDataBytes = Width * (int64)Height * 4;
	
	if ( RawDataBytes > INT32_MAX )
	{
		UE_LOGF(LogImageWrapper, Error, "Bmp dimensions invalid");
		return false;
	}

	RawData.Empty(RawDataBytes);
	RawData.AddUninitialized(RawDataBytes);
	uint8* ImageData = RawData.GetData();
	
	// Copy scanlines, accounting for scanline direction according to the Height field.
	const int32 SrcBytesPerPel = (bmhdr->biBitCount/8);
	check( SrcBytesPerPel*8 == bmhdr->biBitCount );
	const int32 SrcStride = Align(Width*SrcBytesPerPel, 4);
	if ( SrcStride <= 0 )
	{
		UE_LOGF(LogImageWrapper, Error, "Bmp dimensions invalid");
		return false;
	}

	const int64 SrcDataSize = SrcStride * (int64) Height;
	if ( SrcDataSize > (BufferEnd - Bits) )
	{
		UE_LOGF(LogImageWrapper, Error, "Bmp read would overrun buffer");
		return false;
	}
	
	const int32 SrcPtrDiff = bNegativeHeight ? SrcStride : -SrcStride;
	const uint8* SrcPtr = Bits + (bNegativeHeight ? 0 : Height - 1) * SrcStride;

	if ( bmhdr->biBitCount==8)
	{
		// Do palette.

		// If the number for color palette entries is 0, we need to default to 2^biBitCount entries.  In this case 2^8 = 256
		uint32 clrPaletteCount = bmhdr->biClrUsed ? bmhdr->biClrUsed : 256;
		if ( clrPaletteCount > 256 )
		{
			UE_LOGF(LogImageWrapper, Error, "Bmp paletteCount over 256");
			return false;
		}
		
		const uint8* bmpal = AfterHeader;

		if ( clrPaletteCount*4 > (BufferEnd - bmpal) )
		{
			UE_LOGF(LogImageWrapper, Error, "Bmp read would overrun buffer");
			return false;
		}

		TArray<FColor>	Palette;
		Palette.SetNum(256);

		for (uint32 i = 0; i < clrPaletteCount; i++)
		{
			Palette[i] = FColor(bmpal[i * 4 + 2], bmpal[i * 4 + 1], bmpal[i * 4 + 0], 255);
		}

		for(uint32 i= clrPaletteCount; i < 256; i++)
		{
			Palette[i] = FColor(0, 0, 0, 255);
		}
		
		FColor* ImageColors = (FColor*)ImageData;

		for (int32 Y = 0; Y < Height; Y++)
		{
			for (int32 X = 0; X < Width; X++)
			{
				*ImageColors++ = Palette[SrcPtr[X]];
			}

			SrcPtr += SrcPtrDiff;
		}
	}
	else if ( bmhdr->biBitCount==24)
	{
		for (int32 Y = 0; Y < Height; Y++)
		{
			const uint8* SrcRowPtr = SrcPtr;
			for (int32 X = 0; X < Width; X++)
			{
				*ImageData++ = *SrcRowPtr++;
				*ImageData++ = *SrcRowPtr++;
				*ImageData++ = *SrcRowPtr++;
				*ImageData++ = 0xFF;
			}

			SrcPtr += SrcPtrDiff;
		}
	}
	else if ( bmhdr->biBitCount==32 && bmhdr->biCompression == BCBI_RGB)
	{
		// This comment was previously here :
		//  "In BCBI_RGB compression the last 8 bits of the pixel are not used."
		// -> this agrees with MSDN but does not match what Photoshop does in practice
		//  photoshop writes 32-bit ARGB with non-trivial A using BI_RGB
		//	see "porsche512a_ARGB.bmp" also "porsche512a_notadvanced.bmp"
		//	(both the "advanced" and regular photoshop save do this)
		// 

		uint64 TotalA = 0;
		for (int32 Y = 0; Y < Height; Y++)
		{
			// this is just a memcpy, except the accumulation of TotalA
			const uint8* SrcRowPtr = SrcPtr;
			for (int32 X = 0; X < Width; X++)
			{
				*ImageData++ = *SrcRowPtr++;
				*ImageData++ = *SrcRowPtr++;
				*ImageData++ = *SrcRowPtr++;
				TotalA += *SrcRowPtr;
				*ImageData++ = *SrcRowPtr++; // was doing = 0xFF , ignoring fourth byte, as per MSDN
			}

			SrcPtr += SrcPtrDiff;
		}

		if ( TotalA == 0 )
		{
			// assume that this is actually XRGB and they wrote zeros in A
			// go back through and change all A's to 0xFF
			
			ImageData = RawData.GetData();
	
			for (int32 Y = 0; Y < Height; Y++)
			{
				for (int32 X = 0; X < Width; X++)
				{
					ImageData[3] = 0xFF;
					ImageData +=  4;
				}
			}
		}
	}
	else if ( bmhdr->biBitCount==16 && bmhdr->biCompression == BCBI_RGB)
	{
		// 16 bit BI_RGB is 555

		for (int32 Y = 0; Y < Height; Y++)
		{
			for (int32 X = 0; X < Width; X++)
			{
				const uint32 SrcPixel = ((const uint16*)SrcPtr)[X];

				// Set the color values in BGRA order.

				uint32 r = (SrcPixel>>10)&0x1f;;
				uint32 g = (SrcPixel>> 5)&0x1f;
				uint32 b = (SrcPixel    )&0x1f;

				*ImageData++ = (uint8) ( (b<<3) | (b>>2) );
				*ImageData++ = (uint8) ( (g<<3) | (g>>2) );
				*ImageData++ = (uint8) ( (r<<3) | (r>>2) );
				*ImageData++ = 0xFF; // 555 BI_RGB does not use alpha bit
			}

			SrcPtr += SrcPtrDiff;
		}
	}
	else if ( ( bmhdr->biBitCount==16 || bmhdr->biBitCount==32 ) && ( bmhdr->biCompression == BCBI_BITFIELDS || bmhdr->biCompression == BCBI_ALPHABITFIELDS) )
	{
		// Advance past the 40-byte header to get to the color masks :
		//	(note that some bmps have the 52 or 56 byte header with biSize=40 so you cannot check biSize to verify you have valid masks)
		//check( bmhdr->biSize >= 52 );
		//	in theory you could check BitsOffset

		const uint8 * bmhdrEnd;
		if ( ! SafeAdvancePointer(bmhdrEnd, (const uint8 *)bmhdr, BufferEnd, sizeof(FBitmapInfoHeader)) )
		{
			UE_LOGF(LogImageWrapper, Error, "Bmp read would overrun buffer");
			return false;
		}

		// A 52 or 56 byte InfoHeader has masks after a BitmapInfoHeader
		//	a v4 info header also has them in the same place
		//	so reading them from there works in both cases
		const FBmiColorsMask* ColorMask = (FBmiColorsMask*)bmhdrEnd;
		if ( sizeof(FBmiColorsMask) > (BufferEnd - (const uint8 *)ColorMask) )
		{
			UE_LOGF(LogImageWrapper, Error, "Bmp read would overrun buffer");
			return false;
		}

		// Header version 4 introduced the option to declare custom color space, so we can't just assume sRGB past that version.
		//If the header version is V4 or higher we need to make sure we are still using sRGB format
		if (HeaderVersion >= EBitmapHeaderVersion::BHV_BITMAPV4HEADER)
		{
			const FBitmapInfoHeaderV4* bmhdrV4 = (FBitmapInfoHeaderV4*)bmhdr;
			if ( sizeof(FBitmapInfoHeaderV4) > (BufferEnd - (const uint8 *)bmhdrV4) )
			{
				UE_LOGF(LogImageWrapper, Error, "Bmp read would overrun buffer");
				return false;
			}
				
			if (bmhdrV4->biCSType != (uint32)EBitmapCSType::BCST_LCS_sRGB && bmhdrV4->biCSType != (uint32)EBitmapCSType::BCST_LCS_WINDOWS_COLOR_SPACE)
			{
				UE_LOGF(LogImageWrapper, Warning, "BMP uses an unsupported custom color space definition, sRGB color space will be used instead.");
			}
		}

		//Calculating the bit mask info needed to remap the pixels' color values.
		// note RGBAMask[3] can be reading past the end of the header
		//	but we will just read payload bits, and then replace it later when !bHasAlphaChannel
		uint32 RGBAMask[4];
		float MappingRatio[4];
		for (uint32 MaskIndex = 0; MaskIndex < 4; MaskIndex++)
		{
			uint32 Mask = RGBAMask[MaskIndex] = ColorMask->RGBAMask[MaskIndex];
			if ( Mask == 0 )
			{
				MappingRatio[MaskIndex] = 0;
			}
			else
			{
				// count the number of bits on in Mask by counting the zeros on each side:
				int32 TrailingBits = FMath::CountTrailingZeros(Mask);
				int32 NumberOfBits = 32 - (TrailingBits + FMath::CountLeadingZeros(Mask));
				check( NumberOfBits > 0 );

				// note: when NumberOfBits is >= 18, this is not exact (differs from if done in doubles)
				//		but we still get output in [0,255] and the error is small, so let it be
				// use ldexpf to put the >>TrailingBits in the multiply
				MappingRatio[MaskIndex] = ldexpf( (255.f / ((1ULL<<NumberOfBits) - 1) ) , -TrailingBits );
			}
		}

		//In header pre-version 4, we should ignore the last 32bit (alpha) content.
		const bool bHasAlphaChannel = RGBAMask[3] != 0 && HeaderVersion >= EBitmapHeaderVersion::BHV_BITMAPV4HEADER;
		if ( bmhdr->biSize == 56 && RGBAMask[3] != 0 )
		{
			// 56-byte Adobe headers ("bmpv3") might also have valid alpha masks
			//	legacy Unreal import was treating them as bHasAlphaChannel=false
			//	perhaps they should in fact have their alpha mask respected
			// note that Adobe actually uses BI_RGB not BI_BITFIELDS for ARGB output in some cases
			UE_LOGF(LogImageWrapper, Display, "Adobe 56-byte header might have alpha but we ignore it %08X", RGBAMask[3]);				
		}

		float AlphaBias = 0.5f;
		if ( ! bHasAlphaChannel )
		{
			RGBAMask[3] = 0;
			MappingRatio[3] = 0.f;
			AlphaBias = 255.f;
		}
		
		if ( bmhdr->biBitCount == 32 )
		{
			for (int32 Y = 0; Y < Height; Y++)
			{
				for (int32 X = 0; X < Width; X++)
				{
					const uint32 SrcPixel = ((const uint32*)SrcPtr)[X];

					// Set the color values in BGRA order.
					//	integer output of RoundToInt will always fit in U8, no clamp needed
					*ImageData++ = (uint8) ((SrcPixel & RGBAMask[2]) * MappingRatio[2] + 0.5f);
					*ImageData++ = (uint8) ((SrcPixel & RGBAMask[1]) * MappingRatio[1] + 0.5f);
					*ImageData++ = (uint8) ((SrcPixel & RGBAMask[0]) * MappingRatio[0] + 0.5f);
					*ImageData++ = (uint8) ((SrcPixel & RGBAMask[3]) * MappingRatio[3] + AlphaBias);
				}

				SrcPtr += SrcPtrDiff;
			}
		}
		else
		{
			// code dupe: only change from above loop is the type cast on SrcPtr[X]
			check( bmhdr->biBitCount == 16 );

			for (int32 Y = 0; Y < Height; Y++)
			{
				for (int32 X = 0; X < Width; X++)
				{
					const uint32 SrcPixel = ((const uint16*)SrcPtr)[X];

					// Set the color values in BGRA order.
					//	integer output of RoundToInt will always fit in U8, no clamp needed
					*ImageData++ = (uint8) ((SrcPixel & RGBAMask[2]) * MappingRatio[2] + 0.5f);
					*ImageData++ = (uint8) ((SrcPixel & RGBAMask[1]) * MappingRatio[1] + 0.5f);
					*ImageData++ = (uint8) ((SrcPixel & RGBAMask[0]) * MappingRatio[0] + 0.5f);
					*ImageData++ = (uint8) ((SrcPixel & RGBAMask[3]) * MappingRatio[3] + AlphaBias);
				}

				SrcPtr += SrcPtrDiff;
			}
		}
	}
	else
	{
		UE_LOGF(LogImageWrapper, Error, "BMP uses an unsupported format (planes=%i, bitcount=%i, compression=%i)", 
			bmhdr->biPlanes, bmhdr->biBitCount, bmhdr->biCompression);
		return false;
	}

	return true;
}


bool FBmpImageWrapper::SetCompressed(const void* InCompressedData, int64 InCompressedSize)
{
	bool bResult = FImageWrapperBase::SetCompressed(InCompressedData, InCompressedSize);

	bResult = bResult && (bHasHeader ? LoadBMPHeader() : LoadBMPInfoHeader(0));	// Fetch the variables from the header info

	if ( ! bResult )
	{
		CompressedData.Reset();
		return false;
	}
	
	if ( ! FImageCoreUtils::IsImageImportPossible(Width,Height) )
	{
		SetError(TEXT("Image dimensions are not possible to import"));
		return false;
	}

	return bResult;
}


bool FBmpImageWrapper::LoadBMPHeader()
{
	if ( CompressedData.Num() < sizeof(FBitmapInfoHeader) + sizeof(FBitmapFileHeader) )
	{
		UE_LOGF(LogImageWrapper, Error, "Bmp read would overrun buffer");
		return false;
	}

	const FBitmapInfoHeader* bmhdr = (FBitmapInfoHeader *)(CompressedData.GetData() + sizeof(FBitmapFileHeader));
	if ((CompressedData.Num() >= sizeof(FBitmapFileHeader) + sizeof(FBitmapInfoHeader)) && CompressedData.GetData()[0] == 'B' && CompressedData.GetData()[1] == 'M')
	{
		return LoadBMPInfoHeader(sizeof(FBitmapFileHeader));
	}
	
	UE_LOGF(LogImageWrapper, Error, "Bmp header invalid");
	return false;
}


bool FBmpImageWrapper::LoadBMPInfoHeader(int64 HeaderOffset)
{
	if ( CompressedData.Num() < HeaderOffset+(int64)sizeof(FBitmapInfoHeader) )
	{
		UE_LOGF(LogImageWrapper, Error, "Bmp read would overrun buffer");
		return false;
	}

	const FBitmapInfoHeader* bmhdr = (FBitmapInfoHeader *)(CompressedData.GetData() + HeaderOffset);

	if (bmhdr->biCompression != BCBI_RGB && bmhdr->biCompression != BCBI_BITFIELDS && bmhdr->biCompression != BCBI_ALPHABITFIELDS)
	{
		UE_LOGF(LogImageWrapper, Error, "RLE compression of BMP images not supported");
		return false;
	}

	if (bmhdr->biPlanes==1 && (bmhdr->biBitCount==8 || bmhdr->biBitCount==16 || bmhdr->biBitCount==24 || bmhdr->biBitCount==32))
	{
		// Set texture properties.
		Width = bmhdr->biWidth;
		Height = FMath::Abs(bmhdr->biHeight);
		if ( bHalfHeight )
		{
			Height /= 2;
		}
		Format = ERGBFormat::BGRA;
		BitDepth = 8;

		if ( ! BmpDimensionIsValid(Width) || ! BmpDimensionIsValid(Height) )
		{
			UE_LOGF(LogImageWrapper, Error, "Bmp dimensions invalid");
			return false;
		}

		return true;
	}
	else
	{	
		UE_LOGF(LogImageWrapper, Error, "BMP uses an unsupported format (planes = %i, bitcount = %i)", bmhdr->biPlanes, bmhdr->biBitCount);
		return false;
	}
}

bool FBmpImageWrapper::CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const
{
	return ((InFormat == ERGBFormat::BGRA || InFormat == ERGBFormat::Gray) && InBitDepth == 8);
}

ERawImageFormat::Type FBmpImageWrapper::GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const
{
	switch(InFormat)
	{
	case ERawImageFormat::G8:
	case ERawImageFormat::BGRA8:
		return InFormat; // directly supported
	case ERawImageFormat::G16:
		return ERawImageFormat::G8; // needs conversion
	case ERawImageFormat::BGRE8:
	case ERawImageFormat::RGBA16:
	case ERawImageFormat::RGBA16F:
	case ERawImageFormat::RGBA32F:
	case ERawImageFormat::R16F:
	case ERawImageFormat::R32F:
		return ERawImageFormat::BGRA8; // needs conversion
	default:
		check(0);
		return ERawImageFormat::BGRA8;
	};
}


void FBmpImageWrapper::Compress(int32 Quality)
{
	check( Format == ERGBFormat::BGRA || Format == ERGBFormat::Gray );
	check( BitDepth == 8 );

	check( BmpDimensionIsValid(Width) );
	check( BmpDimensionIsValid(Height) );

	// write 8,24, or 32 bit bmp

	int64 NumPixels = Width*(int64)Height;
	int64 RawDataSize = RawData.Num();

	int RawBytesPerPel = (Format == ERGBFormat::BGRA) ? 4 : 1;

	check( RawDataSize == NumPixels*RawBytesPerPel );

	int OutputBytesPerPel = RawBytesPerPel;

	if ( RawBytesPerPel == 4 )
	{
		// scan for A to choose 24 bit output

		const FColor * RawColors = (const FColor *)RawData.GetData();
		bool bHasAnyAlpha = false;
		for(int64 i=0;i<NumPixels;i++)
		{
			if ( RawColors[i].A != 255 )
			{
				bHasAnyAlpha = true;
				break;
			}
		}

		if ( ! bHasAnyAlpha )
		{
			OutputBytesPerPel = 3;
		}
	}

	bool bWritePal = (RawBytesPerPel == 1);

	int OutputRowBytes = (Width*OutputBytesPerPel + 3)&(~3);
	int OutputPalBytes = bWritePal ? 1024 : 0;
	int64 OutputImageBytes = OutputRowBytes * Height;

	CompressedData.Empty( OutputImageBytes + OutputPalBytes + 1024 );

	// scope to write headers:
	{
		// copied from FFileHelper::CreateBitmap
		// 
		// Types.
		#if PLATFORM_SUPPORTS_PRAGMA_PACK
			#pragma pack (push,1)
		#endif
		struct BITMAPFILEHEADER
		{
			uint16 bfType GCC_PACK(1);
			uint32 bfSize GCC_PACK(1);
			uint16 bfReserved1 GCC_PACK(1); 
			uint16 bfReserved2 GCC_PACK(1);
			uint32 bfOffBits GCC_PACK(1);
		} FH = { };
		struct BITMAPINFOHEADER
		{
			uint32 biSize GCC_PACK(1); 
			int32  biWidth GCC_PACK(1);
			int32  biHeight GCC_PACK(1);
			uint16 biPlanes GCC_PACK(1);
			uint16 biBitCount GCC_PACK(1);
			uint32 biCompression GCC_PACK(1);
			uint32 biSizeImage GCC_PACK(1);
			int32  biXPelsPerMeter GCC_PACK(1); 
			int32  biYPelsPerMeter GCC_PACK(1);
			uint32 biClrUsed GCC_PACK(1);
			uint32 biClrImportant GCC_PACK(1); 
		} IH = { };
		struct BITMAPV4HEADER
		{
			uint32 bV4RedMask GCC_PACK(1);
			uint32 bV4GreenMask GCC_PACK(1);
			uint32 bV4BlueMask GCC_PACK(1);
			uint32 bV4AlphaMask GCC_PACK(1);
			uint32 bV4CSType GCC_PACK(1);
			uint32 bV4EndpointR[3] GCC_PACK(1);
			uint32 bV4EndpointG[3] GCC_PACK(1);
			uint32 bV4EndpointB[3] GCC_PACK(1);
			uint32 bV4GammaRed GCC_PACK(1);
			uint32 bV4GammaGreen GCC_PACK(1);
			uint32 bV4GammaBlue GCC_PACK(1);
		} IHV4 = { };
		#if PLATFORM_SUPPORTS_PRAGMA_PACK
			#pragma pack (pop)
		#endif

		bool bInWriteAlpha = ( OutputBytesPerPel == 4 );
		uint32 InfoHeaderSize = sizeof(BITMAPINFOHEADER) + (bInWriteAlpha ? sizeof(BITMAPV4HEADER) : 0);

		// File header.
		FH.bfType       		= INTEL_ORDER16((uint16) ('B' + 256*'M'));
		FH.bfSize       		= INTEL_ORDER32((uint32) (sizeof(BITMAPFILEHEADER) + InfoHeaderSize + OutputImageBytes + OutputPalBytes));
		FH.bfOffBits    		= INTEL_ORDER32((uint32) (sizeof(BITMAPFILEHEADER) + InfoHeaderSize));
		CompressedData.Append( (const uint8 *) &FH, sizeof(FH) );

		// Info header.
		IH.biSize               = INTEL_ORDER32((uint32) InfoHeaderSize);
		IH.biWidth              = INTEL_ORDER32((uint32) Width);
		IH.biHeight             = INTEL_ORDER32((uint32) Height);
		IH.biPlanes             = INTEL_ORDER16((uint16) 1);
		IH.biBitCount           = INTEL_ORDER16((uint16) OutputBytesPerPel * 8);
		if(bInWriteAlpha)
		{
			IH.biCompression    = INTEL_ORDER32((uint32) 3); //BI_BITFIELDS
		}
		else
		{
			IH.biCompression    = INTEL_ORDER32((uint32) 0); //BI_RGB
		}
		IH.biSizeImage          = INTEL_ORDER32((uint32) OutputImageBytes);
		if ( bWritePal )
		{
			IH.biClrUsed            = INTEL_ORDER32((uint32) 256);
			IH.biClrImportant       = INTEL_ORDER32((uint32) 256);
		}
		CompressedData.Append( (const uint8 *) &IH, sizeof(IH) );

		// If we're writing alpha, we need to write the extra portion of the V4 header
		if (bInWriteAlpha)
		{
			IHV4.bV4RedMask     = INTEL_ORDER32((uint32) 0x00ff0000);
			IHV4.bV4GreenMask   = INTEL_ORDER32((uint32) 0x0000ff00);
			IHV4.bV4BlueMask    = INTEL_ORDER32((uint32) 0x000000ff);
			IHV4.bV4AlphaMask   = INTEL_ORDER32((uint32) 0xff000000);
			IHV4.bV4CSType      = INTEL_ORDER32((uint32) 'Win ');
			CompressedData.Append( (const uint8 *) &IHV4, sizeof(IHV4) );
		}
	}

	if ( bWritePal )
	{
		// write palette for G8 :
		FColor Palette[256];

		for(int i=0;i<256;i++)
		{
			Palette[i] = FColor(i,i,i,255);
		}
		
		check( sizeof(Palette) == OutputPalBytes );
		CompressedData.Append( (const uint8 *)Palette,1024 );
	}

	int64 HeaderBytes = CompressedData.Num();
	CompressedData.SetNum( HeaderBytes + OutputImageBytes );
	uint8 * PayloadPtr = CompressedData.GetData() + HeaderBytes;
	
	int OutputRowPadBytes = OutputRowBytes - Width*OutputBytesPerPel;
	check( OutputRowPadBytes < 4 );

	// write rows :
	switch(OutputBytesPerPel)
	{
	case 1:
	{
		const uint8 * RawPtr = RawData.GetData();
		for(int y=Height-1;y>=0;y--)
		{
			memcpy(PayloadPtr,RawPtr + y * Width,Width);
			PayloadPtr += Width;
			memset(PayloadPtr,0,OutputRowPadBytes);
			PayloadPtr += OutputRowPadBytes;
		}
		break;
	}

	case 3:
	{
		const FColor * RawColors = (const FColor *) RawData.GetData();
		for(int y=Height-1;y>=0;y--)
		{
			const FColor * RawRow = RawColors + y * Width;
			for(int x=0;x<Width;x++)
			{
				*PayloadPtr++ = RawRow[x].B;
				*PayloadPtr++ = RawRow[x].G;
				*PayloadPtr++ = RawRow[x].R;
			}
			memset(PayloadPtr,0,OutputRowPadBytes);
			PayloadPtr += OutputRowPadBytes;
		}
		break;
	}

	case 4:
	{
		check( OutputRowBytes == Width * 4 );
		check( OutputRowPadBytes == 0 );

		const uint8 * RawPtr = RawData.GetData();
		for(int y=Height-1;y>=0;y--)
		{
			memcpy(PayloadPtr,RawPtr + y * OutputRowBytes,OutputRowBytes);
			PayloadPtr += OutputRowBytes;
		}
		break;
	}

	default:
		check(0);
		break;
	}
	
	check( PayloadPtr == CompressedData.GetData() + CompressedData.Num() );
}
