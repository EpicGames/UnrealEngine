// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ImageWrapperBase.h"


/**
 * BMP implementation of the helper class
 */
class FBmpImageWrapper
	: public FImageWrapperBase
{
public:

	/** Default Constructor. */
	FBmpImageWrapper(bool bInHasHeader = true, bool bInHalfHeight = false);

public:

	/** Helper function used to uncompress BMP data from a buffer */
	bool UncompressBMPData(const ERGBFormat InFormat, const int32 InBitDepth);

	/** 
	 * Load the header information, returns true if successful.
	 *
	 * @return true if successful
	 */
	bool LoadBMPHeader();

	/** 
	 * Load the sub-header information, returns true if successful.
	 *
	 * @return true if successful
	 */
	bool LoadBMPInfoHeader(int64 HeaderOffset);

public:

	//~ FImageWrapper interface

	virtual void Compress(int32 Quality) override;
	virtual void Uncompress(const ERGBFormat InFormat, int32 InBitDepth) override;
	virtual bool SetCompressed(const void* InCompressedData, int64 InCompressedSize) override;
	
	virtual bool CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const override;
	virtual ERawImageFormat::Type GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const override;

private:

	/** Whether this file has a BMP file header */
	bool bHasHeader;

	/** BMP as a sub-format of ICO stores its height as half their actual size */
	bool bHalfHeight;
};
