// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formats/DdsImageWrapper.h"
#include "ImageWrapperPrivate.h"
#include "ImageCoreUtils.h"

void FDdsImageWrapper::Reset()
{
	Super::Reset();

	FreeDDS();
}

void FDdsImageWrapper::Compress(int32 Quality)
{
	CompressedData.Reset();
	FreeDDS();

	bool bIsExactMatch;
	ERawImageFormat::Type RawImageFormat = GetClosestRawImageFormat(&bIsExactMatch);
	if ( RawImageFormat == ERawImageFormat::Invalid )
	{
		// should not get here if caller checked CanSetRawFormat() like they're supposed to
		SetError(TEXT("Format not supported"));
		check( CompressedData.IsEmpty() ); // signals error
		return;
	}

	// we are not passed SRGB/Gamma info, just assume it is Default for now :
	EGammaSpace GammaSpace = ERawImageFormat::GetDefaultGammaSpace(RawImageFormat);
	
	// some code dupe with IImageWrapper::GetRawImage
	// todo: refactor so this can be shared
	//   after someone does SetRaw() , I should be able to get an FImage view of Raw bits
	//	 for my own writers to use
	//	(can't just use GetRawImage because that's like a GetRaw after SetCompressed)

	FImageView RawImage(RawData.GetData(),Width,Height,1,RawImageFormat,GammaSpace);
	FImage TempImage;

	if ( ! bIsExactMatch )
	{
		// handle non-mapped cases :
	
		switch(Format)
		{
		case ERGBFormat::RGBA:
		{
			// RGBA8 -> BGRA8
			check( BitDepth == 8 );
			check( RawImageFormat == ERawImageFormat::BGRA8 );
			FImageCore::CopyImageRGBABGRA(RawImage, RawImage );
			break;
		}
			
		case ERGBFormat::BGRA:
		{
			// BGRA16 -> RGBA16
			check( BitDepth == 16 );
			check( RawImageFormat == ERawImageFormat::RGBA16 );
			FImageCore::CopyImageRGBABGRA(RawImage, RawImage );
			break;
		}

		case ERGBFormat::GrayF:
		{
			// 1 channel F32 -> 4xF32 :
			check( BitDepth == 32 );
			check( RawImageFormat == ERawImageFormat::RGBA32F );
			const float * Src = (const float *)RawImage.RawData;
			FLinearColor * Dst = (FLinearColor *)TempImage.RawData.GetData();
			int64 NumPixels = RawImage.GetNumPixels();
			for(int64 i=0;i<NumPixels;i++)
			{
				Dst[i] = FLinearColor( Src[i],Src[i],Src[i],1.f );
			}
			RawImage = TempImage;
			break;
		}

		default:
			check(0);
			break;
		}
	}

	// RawImage is ready to write to a DDS
	
	UE::DDS::EDXGIFormat DXGIFormat = UE::DDS::DXGIFormatFromRawFormat(RawImage.Format,RawImage.GammaSpace);

	DDS = UE::DDS::FDDSFile::CreateEmpty2D(Width,Height,1,DXGIFormat,0);

	DDS->FillMip(RawImage,0);
	
	UE::DDS::EDDSError Error = DDS->WriteDDS(CompressedData);
	
	FreeDDS();

	if ( Error != UE::DDS::EDDSError::OK )
	{
		SetError(TEXT("WriteDDS failed"));
		CompressedData.Empty(); // signals error
		return;
	}

	return;
}

bool FDdsImageWrapper::SetCompressed(const void* InCompressedData, int64 InCompressedSize)
{
	Super::SetCompressed(InCompressedData,InCompressedSize);
	FreeDDS();

	UE::DDS::EDDSError Error;
	DDS = UE::DDS::FDDSFile::CreateFromDDSInMemory((const uint8 *)InCompressedData,InCompressedSize,&Error);
	if ( DDS == nullptr || Error != UE::DDS::EDDSError::OK )
	{
		SetError(TEXT("CreateFromDDSInMemory failed"));
		RawData.Empty();
		FreeDDS();
		return false;
	}

	// populate header info :
	
	// change X8 formats to A8 :	
	DDS->ConvertRGBXtoRGBA();
	// change RGBA8 to BGRA8 before DXGIFormatGetClosestRawFormat :
	DDS->ConvertChannelOrder(UE::DDS::EChannelOrder::BGRA);

	// map format to RawImageFormat and ETextureSourceFormat
	ERawImageFormat::Type RawImageFormat = UE::DDS::DXGIFormatGetClosestRawFormat(DDS->DXGIFormat);
	if ( RawImageFormat == ERawImageFormat::Invalid )
	{
		//Warn->Logf(ELogVerbosity::Error, TEXT("DDS DXGIFormat not supported : %d : %s"), (int)DDS->DXGIFormat, UE::DDS::DXGIFormatGetName(DDS->DXGIFormat) );
		
		SetError(TEXT("DDS DXGIFormat not supported"));
		RawData.Empty();
		FreeDDS();
		return false;
	}

	if ( ! DDS->IsValidTexture2D() )
	{
		// @todo Oodle : fix me : FImageWrapperBase::SetError doesn't actually get logged anywhere
		SetError(TEXT("DDS is a complex 3d/cube/array image, only 2d DDS surfaces are supported here"));

		RawData.Empty();
		FreeDDS();
		return false;
	}
	
	// SRGB/Gamma : in some cases we can get SRGB info from the DDS file, so we could report that out
	//		see EditorFactories
	//		not doing for now

	ConvertRawImageFormat(RawImageFormat, Format,BitDepth);

	Width = DDS->Width;
	Height = DDS->Height;

	if ( ! FImageCoreUtils::IsImageImportPossible(Width,Height) )
	{
		SetError(TEXT("Image dimensions are not possible to import"));
		return false;
	}

	return true;
}

void FDdsImageWrapper::Uncompress(const ERGBFormat InFormat, int32 InBitDepth)
{
	RawData.Reset();
	// SetCompressed made the DDS
	check( DDS != nullptr );
	// new style, only support decoding to own format :
	check( InFormat == GetFormat() );
	check( InBitDepth == GetBitDepth() );
		
	ERawImageFormat::Type RawImageFormat = UE::DDS::DXGIFormatGetClosestRawFormat(DDS->DXGIFormat);

	// not setting SRGB/Gamma, just assume it is Default for now :
	FImage RawImage(Width,Height,RawImageFormat);

	if ( ! DDS->GetMipImage(RawImage,0) )
	{
		SetError(TEXT("DDS GetMipImage failed"));
		FreeDDS();		
		check( RawData.IsEmpty() ); // indicates failure
		return;
	}

	RawData = MoveTemp(RawImage.RawData);
}
	
bool FDdsImageWrapper::CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const
{
	// accept any format that can roundtrip through ERawImageFormat :
	// ! exact match is okay, that's an RB swap
	
	ERawImageFormat::Type RawImageFormat = ConvertRGBFormat(InFormat,InBitDepth,nullptr);
	
	ERGBFormat OutFormat;
	int OutBitDepth;
	ConvertRawImageFormat(RawImageFormat, OutFormat,OutBitDepth);

	if ( InFormat == OutFormat && InBitDepth == OutBitDepth )
	{
		return true;
	}
	else
	{
		return false;
	}
}

ERawImageFormat::Type FDdsImageWrapper::GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const
{
	// all raw formats supported
	return InFormat;
}
