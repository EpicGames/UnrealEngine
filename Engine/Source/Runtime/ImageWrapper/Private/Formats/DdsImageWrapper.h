// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ImageWrapperBase.h"
#include "DDSFile.h"

/**
 * DDS implementation of the helper class
 */
class FDdsImageWrapper : public FImageWrapperBase
{
	typedef FImageWrapperBase Super;
public:

	FDdsImageWrapper()
	{
	}
	~FDdsImageWrapper()
	{
		FreeDDS();
	}
	
	void FreeDDS()
	{
		if ( DDS != nullptr )
		{
			delete DDS;
			DDS = nullptr;
		}
	}

public:

	//~ FImageWrapper interface
	
	virtual void Reset() override;

	virtual void Compress(int32 Quality) override;
	virtual bool SetCompressed(const void* InCompressedData, int64 InCompressedSize) override;
	virtual void Uncompress(const ERGBFormat InFormat, int32 InBitDepth) override;
	
	virtual bool CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const override;
	virtual ERawImageFormat::Type GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const override;

public:
	UE::DDS::FDDSFile * DDS = nullptr;
};
