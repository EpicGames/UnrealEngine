// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formats/ExrImageWrapper.h"
#include "ImageWrapperPrivate.h"
#include "Math/GuardedInt.h"

#include "ColorManagement/ColorSpace.h"
#include "Containers/StringConv.h"
#include "HAL/PlatformTime.h"
#include "Math/Float16.h"
#include "ImageCoreUtils.h"

#if WITH_UNREALEXR

FExrImageWrapper::FExrImageWrapper()
	: FImageWrapperBase()
{
}

class FMemFileOut : public Imf::OStream
{
public:
	//-------------------------------------------------------
	// A constructor that opens the file with the given name.
	// The destructor will close the file.
	//-------------------------------------------------------

	FMemFileOut(const char fileName[]) :
		Imf::OStream(fileName),
		Pos(0)
	{
	}

	// InN must be 32bit to match the abstract interface.
	virtual void write(const char c[/*n*/], int32 InN)
	{
		int64 SrcN = (int64)InN;
		int64 DestPost = Pos + SrcN;
		if (DestPost > Data.Num())
		{
			Data.AddUninitialized(FMath::Max(Data.Num() * 2, DestPost) - Data.Num());
		}

		for (int64 i = 0; i < SrcN; ++i)
		{
			Data[Pos + i] = c[i];
		}
		Pos += SrcN;
	}


	//---------------------------------------------------------
	// Get the current writing position, in bytes from the
	// beginning of the file.  If the next call to write() will
	// start writing at the beginning of the file, tellp()
	// returns 0.
	//---------------------------------------------------------

	uint64_t tellp() override
	{
		return Pos;
	}


	//-------------------------------------------
	// Set the current writing position.
	// After calling seekp(i), tellp() returns i.
	//-------------------------------------------

	void seekp(uint64_t pos) override
	{
		Pos = pos;
	}


	int64 Pos;
	TArray64<uint8> Data;
};


class FMemFileIn : public Imf::IStream
{
public:
	//-------------------------------------------------------
	// A constructor that opens the file with the given name.
	// The destructor will close the file.
	//-------------------------------------------------------

	FMemFileIn(const void* InData, int64 InSize)
		: Imf::IStream("")
		, Data((const char *)InData)
		, Size(InSize)
		, Pos(0)
	{
		// InSize is signed but "Size" member is unsigned
		if (InSize < 0)
		{
			Size = 0;
			// MSVC runtime_error vtable size changes based on module bEnableExceptions setting
			//throw std::runtime_error("FMemFileIn: Negative size passed to EXR parser");
			// MSVC std::exception has a char * constructor, but that is not portable, do not use
			throw std::exception();
		}
	}

	//------------------------------------------------------
	// Read from the stream:
	//
	// read(c,n) reads n bytes from the stream, and stores
	// them in array c.  If the stream contains less than n
	// bytes, or if an I/O error occurs, read(c,n) throws
	// an exception.  If read(c,n) reads the last byte from
	// the file it returns false, otherwise it returns true.
	//------------------------------------------------------

	// InN must be 32bit to match the abstract interface.
	virtual bool read (char Out[/*n*/], int32 Count)
	{
		// return false if EOF is hit
		// OpenEXR mostly ignores this return value, you must throw to get error handling

		FGuardedInt64 NextPosition = FGuardedInt64(Pos) + Count;
		if (Count < 0 ||
			NextPosition.InvalidOrGreaterThan(Size))
		{
			//throw std::runtime_error("FMemFileIn: Exr read out of bounds");
			throw std::exception();
		}

		memcpy(Out,Data+Pos,Count);
		Pos += Count; // == NextPosition

		if ( Pos == Size )
		{
			return false;
		}
		else
		{
			return true;
		}
	}

	//--------------------------------------------------------
	// Get the current reading position, in bytes from the
	// beginning of the file.  If the next call to read() will
	// read the first byte in the file, tellg() returns 0.
	//--------------------------------------------------------

	uint64_t tellg() override
	{
		return Pos;
	}

	//-------------------------------------------
	// Set the current reading position.
	// After calling seekg(i), tellg() returns i.
	//-------------------------------------------

	void seekg(uint64_t pos) override
	{
		Pos = pos;
	}

	int64_t size() override
	{
		// Size was verified as non-negative in the constructor.
		return static_cast<int64_t>(Size);
	}

private:

	const char* Data;
	uint64 Size;
	uint64 Pos;
};

// these are the channel names we write
//	we will read whatever channel names are in the file
const char* cChannelNamesRGBA[] = { "R", "G", "B", "A" };
const char* cChannelNamesBGRA[] = { "B", "G", "R", "A" };
//const char* cChannelNamesGray[] = { "G" }; // is that green or gray ?
const char* cChannelNamesGray[] = { "Y" }; // pretty sure "Y" is more standard for gray

static int32 GetChannelNames(ERGBFormat InRGBFormat, const char* const*& OutChannelNames)
{
	int32 ChannelCount;

	switch (InRGBFormat)
	{
	case ERGBFormat::RGBA:
	case ERGBFormat::RGBAF:
		OutChannelNames = cChannelNamesRGBA;
		ChannelCount = UE_ARRAY_COUNT(cChannelNamesRGBA);
		break;

	case ERGBFormat::BGRA:
		OutChannelNames = cChannelNamesBGRA;
		ChannelCount = UE_ARRAY_COUNT(cChannelNamesBGRA);
		break;

	case ERGBFormat::Gray:
	case ERGBFormat::GrayF:
		OutChannelNames = cChannelNamesGray;
		ChannelCount = UE_ARRAY_COUNT(cChannelNamesGray);
		break;

	default:
		OutChannelNames = nullptr;
		ChannelCount = 0;
	}

	return ChannelCount;
}


bool FExrImageWrapper::CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const
{
	// only set float formats
	//	integer formats should be converted before reaching ImageWrapper
	return (InFormat == ERGBFormat::RGBAF || InFormat == ERGBFormat::GrayF) && (InBitDepth == 16 || InBitDepth == 32);
}

ERawImageFormat::Type FExrImageWrapper::GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const
{
	switch(InFormat)
	{
	case ERawImageFormat::RGBA16F:
	case ERawImageFormat::RGBA32F:
	case ERawImageFormat::R16F:
	case ERawImageFormat::R32F:
		return InFormat; // directly supported
	case ERawImageFormat::G8:
		return ERawImageFormat::R16F; // needs conversion
	case ERawImageFormat::BGRA8:
	case ERawImageFormat::BGRE8:
		return ERawImageFormat::RGBA16F; // needs conversion
	case ERawImageFormat::G16:
	case ERawImageFormat::RGBA16:
		return ERawImageFormat::RGBA32F; // needs conversion
	default:
		check(0);
		return ERawImageFormat::BGRA8;
	}
}

bool FExrImageWrapper::SetCompressed(const void* InCompressedData, int64 InCompressedSize)
{
	check(InCompressedData);
	check(InCompressedSize > 0);
	
	// reset variables :
	FileChannelNames.Empty();

	// Check the magic value in advance to avoid spamming the log with EXR parsing errors.
	if (InCompressedSize < sizeof(uint32) || *(uint32*)InCompressedData != Imf::MAGIC)
	{
		return false;
	}

	if (!FImageWrapperBase::SetCompressed(InCompressedData, InCompressedSize))
	{
		return false;
	}

	// openEXR can throw exceptions when parsing invalid data.
	try
	{
		FMemFileIn MemFile(CompressedData.GetData(), CompressedData.Num());
		Imf::InputFile ImfFile(MemFile);
		Imf::Header ImfHeader = ImfFile.header();
		Imf::ChannelList ImfChannels = ImfHeader.channels();
		Imath::Box2i ImfDataWindow = ImfHeader.dataWindow();
		Width = ImfDataWindow.max.x - ImfDataWindow.min.x + 1;
		Height = ImfDataWindow.max.y - ImfDataWindow.min.y + 1;

		bool bHasOnlyHALFChannels = true;
		int32 ChannelCount = 0;

		for (Imf::ChannelList::Iterator Iter = ImfChannels.begin(); Iter != ImfChannels.end(); ++Iter)
		{
			++ChannelCount;

			bHasOnlyHALFChannels = bHasOnlyHALFChannels && Iter.channel().type == Imf::HALF;

			FileChannelNames.Add( MakeUniqueCString(Iter.name()) );
		}

		if ( ChannelCount == 0 )
		{
			SetError( TEXT("EXR has no channels") );
			return false;
		}

		BitDepth = (ChannelCount && bHasOnlyHALFChannels) ? 16 : 32;

		// EXR uint32 channels are currently not supported, therefore input channels are always treated as float channels.
		// Note that channels inside the EXR file are indexed by name, therefore can be decoded in any RGB order.

		if (ChannelCount == 1 )
		{
			Format = ERGBFormat::GrayF;
		}
		else
		{
			Format = ERGBFormat::RGBAF;
		}
	}
	catch (const std::exception& Exception)
	{
		TStringConversion<TStringConvert<char, TCHAR>> Convertor(Exception.what());
		UE_LOGF(LogImageWrapper, Error, "Cannot parse EXR image header: %ls", Convertor.Get());
		SetError(Convertor.Get());
		return false;
	}

	if ( ! FImageCoreUtils::IsImageImportPossible(Width,Height) )
	{
		SetError(TEXT("Image dimensions are not possible to import"));
		return false;
	}

	return true;
}

void FExrImageWrapper::Compress(int32 Quality)
{
	check(RawData.Num());

	// Ensure we haven't already compressed the file.
	if (CompressedData.Num())
	{
		return;
	}

	const double StartTime = FPlatformTime::Seconds();

	Imf::PixelType ImfPixelType;
	TArray64<uint8> ConvertedRawData;
	bool bNeedsConversion = false;

	if (BitDepth == 8)
	{
		// uint8 channels are linearly converted into FFloat16 channels.
		// note: NO GAMMA CORRECTION
		//	(this is no longer used in the modern FImage based APIs, conversion to float should be done before calling this)
		ConvertedRawData.SetNumUninitialized(sizeof(FFloat16) * RawData.Num());
		FFloat16* Output = reinterpret_cast<FFloat16*>(ConvertedRawData.GetData());
		for (int64 i = 0; i < RawData.Num(); ++i)
		{
			Output[i] = FFloat16(RawData[i] / 255.f);
		}
		ImfPixelType = Imf::HALF;
		bNeedsConversion = true;
	}
	else
	{
		ImfPixelType = (BitDepth == 16) ? Imf::HALF : Imf::FLOAT;
	}

	const TArray64<uint8>& PixelData = bNeedsConversion ? ConvertedRawData : RawData;

	const char* const* ChannelNames;
	int32 ChannelCount = GetChannelNames(Format, ChannelNames);
	check((int64)ChannelCount * Width * Height * BitDepth == RawData.Num() * 8);

	int32 BytesPerChannelPixel = (ImfPixelType == Imf::HALF) ? 2 : 4;
	TArray<TArray64<uint8>> ChannelData;
	ChannelData.SetNum(ChannelCount);

	for (int32 c = 0; c < ChannelCount; ++c)
	{
		ChannelData[c].SetNumUninitialized((int64)BytesPerChannelPixel * Width * Height);
	}

	// EXR channels are compressed non-interleaved.
	for (int64 OffsetNonInterleaved = 0, OffsetInterleaved = 0; OffsetInterleaved < PixelData.Num(); OffsetNonInterleaved += BytesPerChannelPixel)
	{
		for (int32 c = 0; c < ChannelCount; ++c)
		{
			for (int32 b = 0; b < BytesPerChannelPixel; ++b, ++OffsetInterleaved)
			{
				ChannelData[c][OffsetNonInterleaved + b] = PixelData[OffsetInterleaved];
			}
		}
	}

	Imf::Compression ImfCompression = (Quality == (int32)EImageCompressionQuality::Uncompressed) ? Imf::Compression::NO_COMPRESSION : Imf::Compression::ZIP_COMPRESSION;
	Imf::Header ImfHeader(Width, Height, 1, Imath::V2f(0, 0), 1, Imf::LineOrder::INCREASING_Y, ImfCompression);
	Imf::FrameBuffer ImfFrameBuffer;

	for (int32 c = 0; c < ChannelCount; ++c)
	{
		ImfHeader.channels().insert(ChannelNames[c], Imf::Channel(ImfPixelType));
		ImfFrameBuffer.insert(ChannelNames[c], Imf::Slice(ImfPixelType, (char*)ChannelData[c].GetData(), BytesPerChannelPixel, (size_t)BytesPerChannelPixel * Width));
	}

	// Write the working color space into EXR chromaticities
	const UE::Color::FColorSpace& WCS = UE::Color::FColorSpace::GetWorking();
	Imf::Chromaticities Chromaticities = {
		IMATH_NAMESPACE::V2f((float)WCS.GetRedChromaticity().X, (float)WCS.GetRedChromaticity().Y),
		IMATH_NAMESPACE::V2f((float)WCS.GetGreenChromaticity().X, (float)WCS.GetGreenChromaticity().Y),
		IMATH_NAMESPACE::V2f((float)WCS.GetBlueChromaticity().X, (float)WCS.GetBlueChromaticity().Y),
		IMATH_NAMESPACE::V2f((float)WCS.GetWhiteChromaticity().X, (float)WCS.GetWhiteChromaticity().Y),
	};
	Imf::addChromaticities(ImfHeader, Chromaticities);

	FMemFileOut MemFile("");
	int64 MemFileLength;

	{
		// This scope ensures that IMF::Outputfile creates a complete file by closing the file when it goes out of scope.
		// To complete the file, EXR seeks back into the file and writes the scanline offsets when the file is closed, 
		// which moves the tellp location. So file length is stored in advance for later use.

		Imf::OutputFile ImfFile(MemFile, ImfHeader, FPlatformMisc::NumberOfCoresIncludingHyperthreads());
		ImfFile.setFrameBuffer(ImfFrameBuffer);
		ImfFile.writePixels(Height);
		MemFileLength = MemFile.tellp();
	}

	CompressedData = MoveTemp(MemFile.Data);
	CompressedData.SetNum(MemFileLength);

	const double DeltaTime = FPlatformTime::Seconds() - StartTime;
	UE_LOGF(LogImageWrapper, Verbose, "Compressed image in %.3f seconds", DeltaTime);
}

void FExrImageWrapper::Uncompress(const ERGBFormat InFormat, const int32 InBitDepth)
{
	check(CompressedData.Num());

	// Ensure we haven't already uncompressed the file.
	if (RawData.Num() && InFormat == Format && InBitDepth == BitDepth)
	{
		return;
	}

	FString ErrorMessage;

	if (InBitDepth == 16 && (InFormat == ERGBFormat::RGBA || InFormat == ERGBFormat::Gray))
	{
		// Before ERGBFormat::RGBAF and ERGBFormat::GrayF were introduced, 16-bit ERGBFormat::RGBA and ERGBFormat::Gray were used to describe float pixel formats.
		// ERGBFormat::RGBA and ERGBFormat::Gray should now only be used for integer channels, while EXR format doesn't support 16-bit integer channels.
		const TCHAR* FormatName = (InFormat == ERGBFormat::RGBA) ? TEXT("RGBA") : TEXT("Gray");
		ErrorMessage = FString::Printf(TEXT("Usage of 16-bit ERGBFormat::%s raw format for decompressing float EXR channels is deprecated, please use ERGBFormat::%sF instead."), FormatName, FormatName);
	}
	else if (InBitDepth != 16 && InBitDepth != 32)
	{
		ErrorMessage = TEXT("Unsupported bit depth, expected 16 or 32.");
	}
	else if (InFormat != ERGBFormat::RGBAF && InFormat != ERGBFormat::GrayF)
	{
		// EXR uint32 channels are currently not supported
		ErrorMessage = TEXT("Unsupported RGB format, expected ERGBFormat::RGBAF or ERGBFormat::GrayF.");
	}

	if (!ErrorMessage.IsEmpty())
	{
		UE_LOGF(LogImageWrapper, Error, "Cannot decompress EXR image: %ls.", *ErrorMessage);
		SetError(*ErrorMessage);
		return;
	}

	// use FileChannelNames[] from SetCompressed
	//	so that we ask for channels that are actually in the file
	const char* ChannelNames[4] = { };
	int32 ChannelCount;
	if ( InFormat == ERGBFormat::GrayF )
	{
		// load the one channel into slot 0 regardless of name
		// output format will be R16F or R32F , so it will look red
		check( FileChannelNames.Num() == 1 );
		ChannelNames[0] = FileChannelNames[0].Get();
		ChannelCount = 1;
	}
	else
	{
		const char * ChannelNamesDesiredChars = "RGBA";
		ChannelCount = 4;

		// try to find channel names in the file which correspond to RGBA
		//	by looking at the last char of the file channel name
		//	and trying to match it to ChannelNamesDesiredChars
		//	(so that eg. "rgb.R" maps to channel 0)
		TArray<const char *> FileChannelNamesUnused;
		for(int FileChannelNamesI=0;FileChannelNamesI<FileChannelNames.Num();FileChannelNamesI++)
		{
			const char * Str = FileChannelNames[FileChannelNamesI].Get();
			check( Str && *Str );
			char LastChar = toupper( Str[ strlen(Str)-1 ] );
			
			for(int ChannelIndex=0;ChannelIndex<4;ChannelIndex++)
			{
				char DesiredChar = ChannelNamesDesiredChars[ChannelIndex];
				if ( LastChar == DesiredChar && ChannelNames[ChannelIndex] == nullptr )
				{
					ChannelNames[ChannelIndex] = Str;
					Str = nullptr;
					break;
				}
			}

			if ( Str )
			{
				FileChannelNamesUnused.Add( Str );
			}
		}

		// if there are channel names in the file which were not mapped
		//	and we have not found 4 channels
		//	then just stuff the file channel names into channels in arbitrary order
		//	eg. if the EXR has channel names like "X","Y","Z" lets go ahead and load them into RGB
		for(int ChannelIndex=0;ChannelIndex<4;ChannelIndex++)
		{
			if ( ChannelNames[ChannelIndex] )
			{
				continue;
			}
			
			if ( ! FileChannelNamesUnused.IsEmpty() )
			{
				ChannelNames[ChannelIndex] = FileChannelNamesUnused.Pop(EAllowShrinking::No);
			}
			else
			{
				// stuff something that is not nullptr
				//	and won't be found in the file
				//	channel will be filled with default value
				const char * DefaultChannelNames[4] =
				{
					"default.R","default.G","default.B","default.A"
				};
				ChannelNames[ChannelIndex] = DefaultChannelNames[ChannelIndex];
			}
		}
				
		UE_LOGF(LogImageWrapper, Verbose, "Reading EXR with Channel Names: %ls %ls %ls %ls", 
			ANSI_TO_TCHAR(ChannelNames[0]),
			ANSI_TO_TCHAR(ChannelNames[1]),
			ANSI_TO_TCHAR(ChannelNames[2]),
			ANSI_TO_TCHAR(ChannelNames[3]));
	}

	check(ChannelCount == 1 || ChannelCount == 4);

	TArray<TArray64<uint8>> ChannelData;
	ChannelData.SetNum(ChannelCount);

	Imf::PixelType ImfPixelType = (InBitDepth == 16) ? Imf::HALF : Imf::FLOAT;
	int32 BytesPerChannelPixel = (ImfPixelType == Imf::HALF) ? 2 : 4;

	// openEXR can throw exceptions when parsing invalid data.
	try
	{
		Imf::FrameBuffer ImfFrameBuffer;
		FMemFileIn MemFile(CompressedData.GetData(), CompressedData.Num());
		Imf::InputFile ImfFile(MemFile);
		Imf::Header ImfHeader = ImfFile.header();
		Imath::Box2i ImfDataWindow = ImfHeader.dataWindow();
		for (int32 c = 0; c < ChannelCount; ++c)
		{
			ChannelData[c].SetNumUninitialized((int64)BytesPerChannelPixel * Width * Height);

			// if you ask for a channel name that is not in the file data, you will get back DefaultValue
			// Use 1.0 as a default value for the alpha channel, in case if it is not present in the EXR, use 0.0 for all other channels.
			bool bIsAlphaChannel = false;
			// only treat a channel named "A" as alpha if it got in slot 3 of RGBA
			//	for example TinyEXR writes out all 1-channel gray images with a channel name "A", we do not treat that as alpha
			if ( c == 3 )
			{
				// should be true for DefaultChannelNames[3]
				bIsAlphaChannel = toupper( ChannelNames[c][ strlen(ChannelNames[c]) -1 ] ) == 'A'; 
			}
			double DefaultValue = bIsAlphaChannel ? 1.0 : 0.0;

			int64 DataWindowOffset = (ImfDataWindow.min.x + ImfDataWindow.min.y * Width) * ((int64)BytesPerChannelPixel);
			check( (ImfDataWindow.max.x - ImfDataWindow.min.x + 1) == Width );
			check( (ImfDataWindow.max.y - ImfDataWindow.min.y + 1) == Height );

			// OpenExr does this offset with the pointers cast to intptr_t ; see ImfFrameBuffer.cpp
			char* ChannelBase = (char*)((intptr_t)ChannelData[c].GetData() - DataWindowOffset);

			ImfFrameBuffer.insert(ChannelNames[c], Imf::Slice(ImfPixelType, ChannelBase, BytesPerChannelPixel, (size_t)BytesPerChannelPixel * Width, 1, 1, DefaultValue));
		}
		ImfFile.setFrameBuffer(ImfFrameBuffer);
		ImfFile.readPixels(ImfDataWindow.min.y, ImfDataWindow.max.y);
	}
	catch (const std::exception& Exception)
	{
		TStringConversion<TStringConvert<char, TCHAR>> Convertor(Exception.what());
		UE_LOGF(LogImageWrapper, Error, "Cannot decompress EXR image: %ls", Convertor.Get());
		SetError(Convertor.Get());
		return;
	}

	// EXR channels are compressed non-interleaved.
	int64 BytesPerChannel = (int64)BytesPerChannelPixel * Width * Height;
	RawData.SetNumUninitialized(BytesPerChannel * ChannelCount);

	const uint8 * ChannelDataPointers[4];
	check(ChannelCount == 1 || ChannelCount == 4);
	for (int32 c = 0; c < ChannelCount; ++c)
	{
		ChannelDataPointers[c] = ChannelData[c].GetData();
	}

	if ( InBitDepth == 16 )
	{
		FFloat16 * Out = (FFloat16 *)&RawData[0];

		for (int64 OffsetNonInterleaved = 0; OffsetNonInterleaved < BytesPerChannel; OffsetNonInterleaved += 2)
		{
			for (int32 c = 0; c < ChannelCount; ++c)
			{
				const FFloat16 * In = (const FFloat16 *)&ChannelDataPointers[c][OffsetNonInterleaved];
				
				*Out = In->GetClampedFinite();

				//check( ! FMath::IsNaN( Out->GetFloat() ) );
				//check( Out->GetFloat() == In->GetFloat() || ! isfinite( In->GetFloat() ) );

				Out++;
			}
		}
	}
	else
	{
		check( InBitDepth == 32 );
		
		float * Out = (float *)&RawData[0];
		
		for (int64 OffsetNonInterleaved = 0; OffsetNonInterleaved < BytesPerChannel; OffsetNonInterleaved += 4)
		{
			for (int32 c = 0; c < ChannelCount; ++c)
			{
				const float * In = (const float *)&ChannelDataPointers[c][OffsetNonInterleaved];
				
				float f = *In;

				// sanitize inf and nan :
				if ( f >= -FLT_MAX && f <= FLT_MAX )
				{
					// finite, leave it
					// nans will fail all compares so not go in here
				}
				else if ( f > FLT_MAX )
				{
					// +inf
					f = FLT_MAX;
				}
				else if ( f < -FLT_MAX )
				{
					// -inf
					f = -FLT_MAX;
				}
				else
				{
					// nan
					f = 0.f;
				}

				*Out = f;
				
				//check( ! FMath::IsNaN( *Out ) );

				Out++;
			}
		}
	}

	Format = InFormat;
	BitDepth = InBitDepth;
}

#elif WITH_UNREALEXR_MINIMAL

FExrImageWrapper::FExrImageWrapper()
	: FImageWrapperBase()
{
}

bool FExrImageWrapper::SetCompressed(const void* InCompressedData, int64 InCompressedSize)
{
	return false;
}

void FExrImageWrapper::Compress(int32 Quality)
{
	check(RawData.Num());

	// Ensure we haven't already compressed the file.
	if (CompressedData.Num())
	{
		return;
	}

	const double StartTime = FPlatformTime::Seconds();

	// conditions enforced by CanSetRawFormat :
	check(BitDepth == 32);
	check(Format == ERGBFormat::RGBAF);

	constexpr uint32 EXRHeaderSize = 313;

	// Indicates the offset, from the start of the file, to get access to the 'full data' of the row
	const uint32 LineOffsetTableSize = 8 * Height; 
	const uint64 TotalHeaderSize = EXRHeaderSize + LineOffsetTableSize;

	// We force use half float as output
	const uint32 PerChannelStride = Width * sizeof(FFloat16);

	// We store RGB, not A
	const uint32 PixelDataRowSize = PerChannelStride * 3; 

	// Full Row is Row Y coordinate + size of pixel data + pixel data
	const uint32 FullDataRowSize = 2 * sizeof(uint32) + PixelDataRowSize; 

	// Final data allocation
	uint32 CompressedDataReservedSize = TotalHeaderSize + Height * FullDataRowSize;
	CompressedData.Reserve(CompressedDataReservedSize);

	auto AddDataU32 = [&](uint32 Value) { CompressedData.Append((uint8*)&Value, sizeof(uint32)); };
	// Based on example header at https://www.openexr.com/documentation/openexrfilelayout.pdf
	{
		auto AddHeaderString = [&](const char* Value) { int32 Len = FCStringAnsi::Strlen(Value); CompressedData.Append((uint8*)Value, Len + 1); };

		// magic number,			  version, flags
		AddDataU32(0x01312f76);	AddDataU32(0x00000002);

		// Attribute name / type / size: 18 bytes per channel + 1 terminating byte
		AddHeaderString("channels"); AddHeaderString("chlist"); AddDataU32(55);
		//name                type (u8,f16,f32)		pLinear/reserved        xSampling          ySampling
		AddHeaderString("B");  AddDataU32(1);     AddDataU32(0);       AddDataU32(1);  AddDataU32(1);
		AddHeaderString("G");  AddDataU32(1);     AddDataU32(0);       AddDataU32(1);  AddDataU32(1);
		AddHeaderString("R");  AddDataU32(1);     AddDataU32(0);       AddDataU32(1);  AddDataU32(1);
		// Separator
		CompressedData.Add(0);

		// Attribute name / type / size: compression expects a single byte
		AddHeaderString("compression"); AddHeaderString("compression"); AddDataU32(1);
		// NO_COMPRESSION = 0, RLE = 1, ZIPS = 2, ZIP = 3, PIZ = 4, PXR24= 5, B44= 6, B44A= 7,
		CompressedData.Add(0);

		// Attribute name / type / size
		AddHeaderString("dataWindow"); AddHeaderString("box2i"); AddDataU32(16);
		// Top Left / Bottom Right
		AddDataU32(0); AddDataU32(0); AddDataU32(Width - 1); AddDataU32(Height - 1);

		// Attribute name / type / size
		AddHeaderString("displayWindow"); AddHeaderString("box2i"); AddDataU32(16);
		AddDataU32(0); AddDataU32(0); AddDataU32(Width - 1); AddDataU32(Height - 1);

		// Attribute name / type / size
		AddHeaderString("lineOrder"); AddHeaderString("lineOrder"); AddDataU32(1);
		// INCREASING_Y = 0, DECREASING_Y = 1, RANDOM_Y = 2
		CompressedData.Add(0);

		// Attribute name / type / size
		AddHeaderString("pixelAspectRatio"); AddHeaderString("float"); AddDataU32(4);
		// 1.0f
		AddDataU32(0x3f800000);

		// Attribute name / type / size
		AddHeaderString("screenWindowCenter"); AddHeaderString("v2f"); AddDataU32(8);
		// 0.0f / 0.0f
		AddDataU32(0); AddDataU32(0);

		// Attribute name / type / size
		AddHeaderString("screenWindowWidth"); AddHeaderString("float"); AddDataU32(4);
		// 1.0f
		AddDataU32(0x3f800000);

		// end of header
		CompressedData.Add(0);

		// Sanity check to make sure header size pre-allocation is still up-to-date
		check(CompressedData.Num() == EXRHeaderSize);
	}

	uint64 LineOffset = TotalHeaderSize;
	// Line Offset table
	for (int32 RowIndex = 0; RowIndex < Height; ++RowIndex)
	{
		CompressedData.Append((uint8*)&LineOffset, sizeof(uint64));
		LineOffset += FullDataRowSize;
	}
	check(CompressedData.Num() == TotalHeaderSize);

	// Raw Data output
	const FLinearColor* SrcPixelData = (FLinearColor*)RawData.GetData();
	for (int32 RowIndex = 0; RowIndex < Height; ++RowIndex, SrcPixelData += Width)
	{
		uint32 OutEXRFileDataRowBeginSize = CompressedData.Num();

		AddDataU32(RowIndex);
		AddDataU32(PixelDataRowSize);

		// Layout is BBBB..BBGGGG..GGRRRR..RR
		for (int32 ColIndex = 0; ColIndex < Width; ++ColIndex)
		{
			FFloat16 B16(SrcPixelData[ColIndex].B);
			CompressedData.Append((uint8*)&B16.Encoded, sizeof(B16.Encoded));
		}

		for (int32 ColIndex = 0; ColIndex < Width; ++ColIndex)
		{
			FFloat16 G16(SrcPixelData[ColIndex].G);
			CompressedData.Append((uint8*)&G16.Encoded, sizeof(G16.Encoded));
		}

		for (int32 ColIndex = 0; ColIndex < Width; ++ColIndex)
		{
			FFloat16 R16(SrcPixelData[ColIndex].R);
			CompressedData.Append((uint8*)&R16.Encoded, sizeof(R16.Encoded));
		}

		check(CompressedData.Num() == OutEXRFileDataRowBeginSize + FullDataRowSize);
	}

	// make sure we have written all the data we reserved in the first place
	check(CompressedData.Num() == CompressedDataReservedSize);

	const double DeltaTime = FPlatformTime::Seconds() - StartTime;
	UE_LOGF(LogImageWrapper, Verbose, "written image in %.3f seconds", DeltaTime);
}

void FExrImageWrapper::Uncompress(const ERGBFormat InFormat, const int32 InBitDepth)
{
	ensure(false);
	UE_LOGF(LogImageWrapper, Error, "FExrImageWrapper::Uncompress is not supported");
}

bool FExrImageWrapper::CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const
{
	return (InFormat == ERGBFormat::RGBAF) && (InBitDepth == 32);
}

ERawImageFormat::Type FExrImageWrapper::GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const
{
	return ERawImageFormat::RGBA32F;
}

#endif // WITH_UNREALEXR
