// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

#include "ImageWrapperBase.h"

#if WITH_UNREALEXR || WITH_UNREALEXR_MINIMAL

#if WITH_UNREALEXR
THIRD_PARTY_INCLUDES_START
	#include "Imath/ImathBox.h"
	#include "OpenEXR/ImfArray.h"
	#include "OpenEXR/ImfChannelList.h"
	#include "OpenEXR/ImfHeader.h"
	#include "OpenEXR/ImfIO.h"
	#include "OpenEXR/ImfInputFile.h"
	#include "OpenEXR/ImfOutputFile.h"
	#include "OpenEXR/ImfRgbaFile.h"
	#include "OpenEXR/ImfStandardAttributes.h"
	#include "OpenEXR/ImfStdIO.h"
	#include "OpenEXR/ImfVersion.h"
THIRD_PARTY_INCLUDES_END
#endif

/**
 * OpenEXR implementation of the helper class
 */
class FExrImageWrapper
	: public FImageWrapperBase
{
public:

	/**
	 * Default Constructor.
	 */
	FExrImageWrapper();

public:

	//~ FImageWrapper interface

	virtual bool SetCompressed(const void* InCompressedData, int64 InCompressedSize) override;
	virtual void Compress(int32 Quality) override;
	virtual void Uncompress(const ERGBFormat InFormat, int32 InBitDepth) override;
	
	virtual bool CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const override;
	virtual ERawImageFormat::Type GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const override;

	typedef TUniquePtr<char[]> FUniqueCString;
	FUniqueCString MakeUniqueCString(const char *str)
	{
		size_t num = strlen(str)+1;
		FUniqueCString Ret = MakeUnique<char[]>( num );
		memcpy(Ret.Get(),str,num);
		return Ret;
	}
	TArray< FUniqueCString > FileChannelNames;
};

#endif // WITH_UNREALEXR
