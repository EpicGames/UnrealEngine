// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formats/HdrImageWrapper.h"
#include "ImageWrapperPrivate.h"
#include "ImageCoreUtils.h"

#define LOCTEXT_NAMESPACE "HdrImageWrapper"

namespace UE::ImageWrapper::Private::HdrImageWrapper
{
	FText GetMalformedHeaderErrorMessage()
	{
		return LOCTEXT("MalformedHeader", "The file header is malformed. The HDR image is likely corrupted.");
	}

	FText GetEndOfBufferErrorMessage()
	{
		return LOCTEXT("EndOFBufferError", "Reached the end of the buffer before finishing decompressing the HDR. The HDR image is likely corrupted.");
	}

	FText GetMalformedScanlineErrorMessage()
	{
		return LOCTEXT("MalformedScanline", "Compressed data for HDR scanline is malformed. The HDR image is likely to be corrupted.");
	}
}


bool FHdrImageWrapper::SetCompressedFromView(TArrayView64<const uint8> Data)
{
	// CompressedData is just a View, does not take a copy
	CompressedData = Data;

	if (CompressedData.Num() < 11)
	{
		return FailHeaderParsing();
	}

	const uint8* FileDataPtr = CompressedData.GetData();
	char Line[256];

	if (!GetHeaderLine(FileDataPtr, Line))
	{
		return FailHeaderParsing();
	}

	if (FCStringAnsi::Strcmp(Line, "#?RADIANCE") != 0 &&
		FCStringAnsi::Strcmp(Line, "#?RGBE") != 0)
	{
		return FailHeaderParsing();
	}

	// Read header lines: free-form, keep going until we hit a blank line
	bool bHasFormat = false;

	for (;;)
	{
		if (!GetHeaderLine(FileDataPtr, Line))
		{
			return FailHeaderParsing();
		}

		// Blank line denotes end of header
		if (!Line[0])
			break;

		const char* Cursor = Line;

		// Format specified?
		if (ParseMatchString(Cursor, "FORMAT="))
		{
			bHasFormat = true;

			// Currently we only support RGBE
			if (FCStringAnsi::Strcmp(Cursor, "32-bit_rle_rgbe") != 0)
			{
				SetAndLogError(LOCTEXT("WrongFormatError", "The HDR image uses an unsupported format. Only the 32-bit_rle_rgbe format is supported."));
				FreeCompressedData();
				return false;
			}
		}
	}

	// If we got through the header without it mentioning a format, the file is malformed.
	if (!bHasFormat)
	{
		return FailHeaderParsing();
	}

	// Read one more line which specifies the resolution
	if (!GetHeaderLine(FileDataPtr, Line))
	{
		return FailHeaderParsing();
	}

	int ImageWidth;
	int ImageHeight;
	if ( !ParseImageSize(Line, &ImageWidth, &ImageHeight) ||
		ImageWidth <= 0 || ImageHeight <= 0 )
	{
		// If we don't like the resolution line (our parser is very strict), log what it was
		// as a breadcrumb for debugging.
		FString BadResolutionLine(Line);
		UE_LOGF(LogImageWrapper, Display, "HDR bad resolution line was: \"%ls\"", *BadResolutionLine);

		SetAndLogError(LOCTEXT("InvalidSizeError", "The HDR image specifies an invalid size."));
		FreeCompressedData();
		return false;
	}

	Width = ImageWidth;
	Height = ImageHeight;
	RGBDataStart = FileDataPtr;
	
	if ( ! FImageCoreUtils::IsImageImportPossible(Width,Height) )
	{
		SetAndLogError(LOCTEXT("ImpossibleImport","Image dimensions are not possible to import"));
		return false;
	}

	return true;
}

bool FHdrImageWrapper::SetCompressed(const void* InCompressedData, int64 InCompressedSize)
{
	// takes copy
	CompressedDataHolder.Reset(InCompressedSize);
	CompressedDataHolder.Append((const uint8*)InCompressedData, InCompressedSize);

	return SetCompressedFromView(MakeArrayView(CompressedDataHolder));
}

// CanSetRawFormat returns true if SetRaw will accept this format
bool FHdrImageWrapper::CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const
{
	return InFormat == ERGBFormat::BGRE && InBitDepth == 8;
}

// returns InFormat if supported, else maps to something supported
ERawImageFormat::Type FHdrImageWrapper::GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const
{
	// only writes one format :
	return ERawImageFormat::BGRE8;
}

bool FHdrImageWrapper::SetRaw(const void* InRawData, int64 InRawSize, const int32 InWidth, const int32 InHeight, const ERGBFormat InFormat, const int32 InBitDepth, const int32 InBytesPerRow)
{
	if ( ! CanSetRawFormat(InFormat,InBitDepth) )
	{
		UE_LOGF(LogImageWrapper, Warning, "ImageWrapper unsupported format; check CanSetRawFormat; %d x %d", (int)InFormat,InBitDepth);
		return false;
	}

	if (InWidth <= 0 || InHeight <= 0)
	{
		UE_LOGF(LogImageWrapper, Warning, "ImageWrapper HDR unsupported size %d x %d", InWidth, InHeight);
		return false;
	}

	RawDataHolder.Empty(InRawSize);
	RawDataHolder.Append((const uint8 *)InRawData,InRawSize);
	Width = InWidth;
	Height = InHeight;

	check( InFormat == ERGBFormat::BGRE );
	check( InBitDepth == 8 );

	return true;
}

TArray64<uint8> FHdrImageWrapper::GetCompressed(int32 Quality)
{
	// must have set BGRE8 raw data :
	int64 NumPixels = (int64)Width * Height;
	check( RawDataHolder.Num() == NumPixels * 4 );
	
	char Header[MAX_SPRINTF];
	int32 HeaderLen = FCStringAnsi::Sprintf(Header, "#?RADIANCE\nFORMAT=32-bit_rle_rgbe\n\n-Y %d +X %d\n", Height, Width);
	if (HeaderLen <= 0) // Error during Sprintf for whatever reason?
	{
		return TArray64<uint8>();
	}

	FreeCompressedData();
	TArray64<uint8> CompressedDataArray;
	CompressedDataArray.SetNumUninitialized(HeaderLen + NumPixels * 4);
	uint8 * CompressedPtr = CompressedDataArray.GetData();

	memcpy(CompressedPtr,Header,HeaderLen);
	CompressedPtr += HeaderLen;

	// just put the BGRE8 bytes
	// but need to swizzle to RGBE8 order:
	const uint8* FromBytes = RawDataHolder.GetData();
	for(int64 i=0;i<NumPixels;i++)
	{
		CompressedPtr[i * 4 + 0] = FromBytes[i * 4 + 2];
		CompressedPtr[i * 4 + 1] = FromBytes[i * 4 + 1];
		CompressedPtr[i * 4 + 2] = FromBytes[i * 4 + 0];
		CompressedPtr[i * 4 + 3] = FromBytes[i * 4 + 3];
	}

	return MoveTemp(CompressedDataArray);
}

bool FHdrImageWrapper::GetRaw(const ERGBFormat InFormat, int32 InBitDepth, TArray64<uint8>& OutRawData)
{
	if (InFormat != ERGBFormat::BGRE || InBitDepth != 8)
	{
		SetAndLogError(LOCTEXT("UnSupportedFormatORBitDepth", "The format and/or the bit depth is not supported by the HdrImageWrapper. Only the BGRE format and a bitdepth of 8 is supported"));
		return false;
	}

	if (!IsCompressedImageValid())
	{
		return false;
	}

	const int64 SizeRawImageInBytes = (int64)Width * Height * 4;
	OutRawData.Reset(SizeRawImageInBytes);
	OutRawData.AddUninitialized(SizeRawImageInBytes);

	const uint8* FileDataPtr = RGBDataStart;
	const uint8* FileDataEnd = CompressedData.GetData() + CompressedData.Num();

	for (int32 Y = 0; Y < Height; ++Y)
	{
		if (!DecompressScanline(&(OutRawData[(int64)Width * Y * 4]), FileDataPtr, FileDataEnd))
		{
			OutRawData.Empty();
			return false;
		}
	}

	return true;
}

IMAGEWRAPPER_API bool FHdrImageWrapper::GetRaw(const ERGBFormat InFormat, int32 InBitDepth, FDecompressedImageOutput& OutDecompressedImage)
{
	TArray64<uint8> OutRawBuffer;
	if (!GetRaw(InFormat, InBitDepth, OutRawBuffer))
	{
		return false;
	}
	OutDecompressedImage.MipMapImage.AddMipImage(MoveTemp(OutRawBuffer), GetWidth(), GetHeight());
	return true;
}

int64 FHdrImageWrapper::GetWidth() const
{
	return Width;
}

int64 FHdrImageWrapper::GetHeight() const
{
	return Height;
}

int32 FHdrImageWrapper::GetBitDepth() const
{
	return 8;
}

ERGBFormat FHdrImageWrapper::GetFormat() const
{
	return ERGBFormat::BGRE;
}

const FText& FHdrImageWrapper::GetErrorMessage() const
{
	return ErrorMessage;
}

void FHdrImageWrapper::SetAndLogError(const FText& InText)
{
	ErrorMessage = InText;
	UE_LOGF(LogImageWrapper, Error, "%ls", *InText.ToString());
}

bool FHdrImageWrapper::FailHeaderParsing()
{
	SetAndLogError(UE::ImageWrapper::Private::HdrImageWrapper::GetMalformedHeaderErrorMessage());
	FreeCompressedData();
	return false;
}

bool FHdrImageWrapper::FailUnexpectedEOB()
{
	SetAndLogError(UE::ImageWrapper::Private::HdrImageWrapper::GetEndOfBufferErrorMessage());
	return false;
}

bool FHdrImageWrapper::FailMalformedScanline()
{
	SetAndLogError(UE::ImageWrapper::Private::HdrImageWrapper::GetMalformedScanlineErrorMessage());
	return false;
}

bool FHdrImageWrapper::GetHeaderLine(const uint8*& BufferPos, char Line[256])
{
	const uint8* EndOfBuffer = CompressedData.GetData() + CompressedData.Num();

	for(int i = 0; i < 256; ++i)
	{
		// May never read past end of buffer
		check(BufferPos <= EndOfBuffer); // == may happen, > is a bug.

		if (BufferPos >= EndOfBuffer)
		{
			return false;
		}

		if (*BufferPos == 0)
		{
			// We don't allow NUL characters in header, that's malformed
			return false;
		}
		else if (*BufferPos == '\n') // Line break -> end of line in HDR (but CRs are not significant!)
		{
			// Terminate the line, we're good to go, but do consume the newline.
			// We insert the terminating 0 here, and this is the only path where we return true,
			// thus guaranteeing that successfully returned lines are 0-terminated.
			BufferPos++;
			Line[i] = 0;
			return true;
		}

		// All other characters go into the line verbatim.
		Line[i] = *BufferPos++;
	}

	// Falling out of the loop after reading 256 chars, we consider a malformed line.
	return false;
}

// InOutCursor points to current parse cursor, into a 0-terminated string.
// InExpected is a 0-terminated string that we expect an exact match to.
//
// On success, return true and point InOutCursor at the end of the matched string.
// On failure, return false and leave InOutCursor where it is.
bool FHdrImageWrapper::ParseMatchString(const char*& InOutCursor, const char* InExpected)
{
	const char* Cursor = InOutCursor;

	while (*InExpected)
	{
		if (*Cursor != *InExpected)
		{
			return false;
		}

		// The two bytes match, advance.
		++Cursor;
		++InExpected;
	}

	InOutCursor = Cursor;
	return true;
}

// Like atoi, but we return the final cursor, have an error reporting mechanism
// for overflows, and don't accept signed numbers (or '+' signs for that matter).
//
// InOutCursor points into a properly 0-terminated string, so we can just keep
// reading while chars are between '0' and '9' (since NUL is not).
bool FHdrImageWrapper::ParsePositiveInt(const char*& InOutCursor, int* OutValue)
{
	// We require the string to start with a digit
	if (*InOutCursor < '0' || *InOutCursor > '9')
	{
		return false;
	}

	int Value = *InOutCursor - '0'; // can't overflow.
	++InOutCursor;

	// Keep consuming digits in the digit string
	while (*InOutCursor >= '0' && *InOutCursor <= '9')
	{
		int Digit = *InOutCursor - '0';
		++InOutCursor;

		int64 NewValue = (int64)Value * 10 + Digit;

		// Overflow?
		if (NewValue > TNumericLimits<int>::Max())
		{
			return false;
		}

		// We just checked it's in range.
		Value = (int)NewValue;
	}

	*OutValue = Value;
	return true;
}

// InLine is known 0-terminated.
bool FHdrImageWrapper::ParseImageSize(const char* InLine, int* OutWidth, int* OutHeight)
{
	// We only support the (default) -Y <height> +X <width> form of the image size.
	if (!ParseMatchString(InLine, "-Y "))
	{
		return false;
	}

	if (!ParsePositiveInt(InLine, OutHeight))
	{
		return false;
	}

	if (!ParseMatchString(InLine, " +X "))
	{
		return false;
	}

	if (!ParsePositiveInt(InLine, OutWidth))
	{
		return false;
	}

	return *InLine == 0; // All bytes in line must be consumed
}

// Trivial helper func to make repetitive error checks easier to read.
// InCursor <= InEnd is assumed; check that we can read InAmount more bytes without
// going past InEnd.
bool FHdrImageWrapper::HaveBytes(const uint8* InCursor, const uint8* InEnd, int InAmount)
{
	// InCursor <= InEnd; InEnd - InCursor gives us the number of bytes left.
	// (Do it this way instead of "InCursor + InAmount <= InEnd" because the latter
	// might overflow.)
	return InEnd - InCursor >= InAmount;
}

bool FHdrImageWrapper::DecompressScanline(uint8* Out, const uint8*& In, const uint8* InEnd)
{
	// minimum and maximum scanline length for encoding
	const int32 MINELEN = 8;
	const int32 MAXELEN = 0x7fff;

	if (Width < MINELEN || Width > MAXELEN)
	{
		return OldDecompressScanline(Out, In, InEnd, Width, false);
	}

	if (!HaveBytes(In, InEnd, 1))
	{
		return false;
	}

	uint8 Red = *In;

	if(Red != 2)
	{
		return OldDecompressScanline(Out, In, InEnd, Width, false);
	}

	++In;

	if (!HaveBytes(In, InEnd, 3))
	{
		return false;
	}

	uint8 Green = *In++;
	uint8 Blue = *In++;
	uint8 Exponent = *In++;

	if(Green != 2 || (Blue & 128))
	{
		*Out++ = Blue;
		*Out++ = Green;
		*Out++ = Red;
		*Out++ = Exponent;
		return OldDecompressScanline(Out, In, InEnd, Width - 1, true);
	}

	for(uint32 ChannelRead = 0; ChannelRead < 4; ++ChannelRead)
	{
		// The file is in RGBE but we decompress in BGRE So swap the red and blue
		uint8 CurrentToWrite = ChannelRead;
		if (ChannelRead == 0)
		{
			CurrentToWrite = 2;
		}
		else if (ChannelRead == 2)
		{
			CurrentToWrite = 0;
		}

		const uint8* LocalIn = In;
		uint8* OutSingleChannel = Out + CurrentToWrite;
		int32 MultiRunIndex = 0;

		while ( MultiRunIndex < Width )
		{
			if (!HaveBytes(LocalIn, InEnd, 1))
			{
				return FailUnexpectedEOB();
			}

			uint8 Current = *LocalIn++;

			if (Current > 128)
			{
				// Actual run
				int Count = Current & 0x7f;

				if (!HaveBytes(LocalIn, InEnd, 1))
				{
					return FailUnexpectedEOB();
				}
				Current = *LocalIn++;

				// Run needs to stay within scan line.
				if (Width - MultiRunIndex < Count)
				{
					return FailMalformedScanline();
				}

				for(int RunIndex = 0; RunIndex < Count; ++RunIndex)
				{
					*OutSingleChannel = Current;
					OutSingleChannel += 4;
				}
				MultiRunIndex += Count;
			}
			else
			{
				// Literal run.
				int Count = Current;

				// Do one check up front whether we have enough data bytes following
				if (!HaveBytes(LocalIn, InEnd, Count))
				{
					return FailUnexpectedEOB();
				}

				// Literal run needs to stay within scan line.
				if (Width - MultiRunIndex < Count)
				{
					return FailMalformedScanline();
				}

				for(int RunIndex = 0; RunIndex < Count; ++RunIndex)
				{
					// All buffer checks were done up front.
					*OutSingleChannel = *LocalIn++;
					OutSingleChannel += 4;
				}
				MultiRunIndex += Count;
			}
		}

		In = LocalIn;
	}

	return true;
}

bool FHdrImageWrapper::OldDecompressScanline(uint8* Out, const uint8*& InCodedScanline, const uint8* InEnd, int32 Length, bool bInitialRunAllowed)
{
	const uint8* In = InCodedScanline; // Copy to local var
	int32 Shift = 0;

	// If an initial run is not allowed, set Shift to 32, which will make us fail if the first thing we
	// see in this scanline is a run, but will be cleared after the first non-run pixel.
	if (!bInitialRunAllowed)
	{
		Shift = 32;
	}

	while (Length > 0)
	{
		if (!HaveBytes(In, InEnd, 4))
		{
			return FailUnexpectedEOB();
		}

		uint8 Red = *In++; 
		uint8 Green = *In++;
		uint8 Blue = *In++; 
		uint8 Exponent = *In++; 

		if(Red == 1 && Green == 1 && Blue == 1)
		{
			// It's not illegal to hit Shift=32 (say after a non-trivial run with Shift=24 in a giant image),
			// but since we have a 32-bit width limit, there is just no legitimate reason to have another run
			// once Shift=32, it should always be followed by literals.
			//
			// We also use this to catch runs at the start of a scanline when there's no pixels to repeat yet,
			// by initializing Shift=32 on the first pixel of a scanline (we sometimes get called with one pixel
			// already decoded, so this is conditional).
			if (Shift >= 32)
			{
				return FailMalformedScanline();
			}

			// Doing Count calculation in 64 bits to avoid overflow concerns when Shift=24.
			int64 Count = (int64)Exponent << Shift;
			if (Count > Length)
			{
				return FailMalformedScanline();
			}

			Length -= Count;

			// Read previous pixel. See comments on top of function and handling of Shift >= 32 above for why
			// we are guaranteed to have a previous pixel in the scanline when we get here.
			Red = *(Out - 4); 
			Green = *(Out - 3);
			Blue = *(Out - 2); 
			Exponent = *(Out - 1); 

			while (Count > 0)
			{
				*Out++ = Blue;
				*Out++ = Green;
				*Out++ = Red;
				*Out++ = Exponent;
				--Count;
			}

			Shift += 8;
		}
		else
		{
			*Out++ = Blue;
			*Out++ = Green;
			*Out++ = Red;
			*Out++ = Exponent;
			Shift = 0;
			--Length;
		}
	}

	// On successful decode, copy read cursor back
	InCodedScanline = In;
	return true;
}

bool FHdrImageWrapper::IsCompressedImageValid() const
{
	return CompressedData.Num() > 0 && RGBDataStart;
}

void FHdrImageWrapper::FreeCompressedData()
{
	CompressedData = TArrayView64<const uint8>();
	RGBDataStart = nullptr;
	CompressedDataHolder.Empty();
}

bool FHdrImageWrapper::SupportsMetadata() const
{
	return false;
}

void FHdrImageWrapper::AddMetadata(const FString&, const FString&)
{
}

void FHdrImageWrapper::GetMetadata(TStringMap& OutMetadata) const
{
	OutMetadata.Reset();
}

#undef LOCTEXT_NAMESPACE
