// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formats/IcnsImageWrapper.h"
#include "ImageWrapperPrivate.h"
#include "ImageCoreUtils.h"

#if PLATFORM_MAC
#include "Apple/ScopeAutoreleasePool.h"
#include "Mac/MacSystemIncludes.h"
#endif


/* FIcnsImageWrapper structors
 *****************************************************************************/

FIcnsImageWrapper::FIcnsImageWrapper()
	: FImageWrapperBase()
{ }


/* FImageWrapper interface
 *****************************************************************************/

bool FIcnsImageWrapper::SetCompressed(const void* InCompressedData, int64 InCompressedSize)
{
#if PLATFORM_MAC
	if ( ! FImageWrapperBase::SetCompressed(InCompressedData, InCompressedSize) )
	{
		return false;
	}
	
	// set image properties from header

	// always read as BGRA8 regardless of the image format in the file :
	Format = ERGBFormat::BGRA;
	BitDepth = 8;
	Width = Height = 0;

	// get width and height from image data :
	{
		SCOPED_AUTORELEASE_POOL;

		NSData* ImageData = [NSData dataWithBytesNoCopy:CompressedData.GetData() length:CompressedData.Num() freeWhenDone:NO];
		NSImage* Image = [[NSImage alloc] initWithData:ImageData];
		if (Image)
		{
			Width = Image.size.width;
			Height = Image.size.height;

			// TODO: We have decoded the image this far we might want to store this data/object so ::Uncompress doesn't have to do it again
			[Image release];
		}
	}
	
	if ( Width == 0 )
	{
		return false;
	}

	if ( ! FImageCoreUtils::IsImageImportPossible(Width,Height) )
	{
		SetError(TEXT("Image dimensions are not possible to import"));
		return false;
	}

	return true;
#else
	return false;
#endif
}


// CanSetRawFormat returns true if SetRaw will accept this format
bool FIcnsImageWrapper::CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const
{
	//checkf(false, TEXT("ICNS compression not supported"));
	return false;
}

// returns InFormat if supported, else maps to something supported
ERawImageFormat::Type FIcnsImageWrapper::GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const
{
	//checkf(false, TEXT("ICNS compression not supported"));
	return ERawImageFormat::BGRA8;
}

bool FIcnsImageWrapper::SetRaw(const void* InRawData, int64 InRawSize, const int32 InWidth, const int32 InHeight, const ERGBFormat InFormat, const int32 InBitDepth, const int32 InBytesPerRow)
{
	// Only support tightly packed
	check(InBytesPerRow == 0);

#if PLATFORM_MAC
	// CanSetRawFormat returns false, this will always fail, remove?
	return FImageWrapperBase::SetRaw(InRawData, InRawSize, InWidth, InHeight, InFormat, InBitDepth, InBytesPerRow);
#else
	return false;
#endif
}


void FIcnsImageWrapper::Compress(int32 Quality)
{
	checkf(false, TEXT("ICNS compression not supported"));
}


void FIcnsImageWrapper::Uncompress(const ERGBFormat InFormat, const int32 InBitDepth)
{
#if PLATFORM_MAC
	SCOPED_AUTORELEASE_POOL;

	NSData* ImageData = [NSData dataWithBytesNoCopy:CompressedData.GetData() length:CompressedData.Num() freeWhenDone:NO];
	NSImage* Image = [[NSImage alloc] initWithData:ImageData];
	if (Image)
	{
		NSBitmapImageRep* Bitmap = [NSBitmapImageRep imageRepWithData:[Image TIFFRepresentation]];
		if (Bitmap)
		{
			check(InFormat == ERGBFormat::BGRA || InFormat == ERGBFormat::RGBA);
			check(InBitDepth == 8);
			
			Format = InFormat;
			BitDepth = InBitDepth;

			Width = [Bitmap pixelsWide];
			Height = [Bitmap pixelsHigh];

			// InBitDepth of 8 above must be the per channel value not the image pixel bit depth
			const size_t SrcBitDepth = (Bitmap.bitsPerPixel / Bitmap.samplesPerPixel);
			check(SrcBitDepth == 8);
			
			// Non compressed total image size
			const size_t SrcImageSize = Width * Height * Bitmap.samplesPerPixel;
			
			// 4 channel output BGRA or RGBA
			const size_t DestBytesPerPixel = 4;
			const size_t DestImageSize = Width * Height * DestBytesPerPixel;
			
			RawData.Empty();
			RawData.AddUninitialized(DestImageSize);
			
			if(SrcImageSize == DestImageSize && Bitmap.bytesPerPlane == SrcImageSize && Bitmap.numberOfPlanes == 1)
			{
				// Exact match direct copy
				FMemory::Memcpy(RawData.GetData(), [Bitmap bitmapData], DestImageSize);
			}
			else if(Bitmap.bytesPerPlane == SrcImageSize && Bitmap.numberOfPlanes == 1)
			{
				// Manual copy, could be 24bit or gray scale.  Be conservative about the number of availble source channels during the copy
				uint8* SrcData = [Bitmap bitmapData];
				
				#define SRC_IMAGE_INDEX ((Y * Bitmap.bytesPerRow) + (X * Bitmap.samplesPerPixel))
				#define DEST_IMAGE_INDEX ((Y * (DestBytesPerPixel * Width)) + (X * DestBytesPerPixel))
				
				for(size_t Y = 0;Y < Height;++Y)
				{
					if(Bitmap.samplesPerPixel >= 4)
					{
						for(size_t X = 0;X < Width;++X)
						{
							size_t SrcIndex = SRC_IMAGE_INDEX;
							size_t DestIndex = DEST_IMAGE_INDEX;

							RawData[DestIndex + 0] = SrcData[SrcIndex + 0];
							RawData[DestIndex + 1] = SrcData[SrcIndex + 1];
							RawData[DestIndex + 2] = SrcData[SrcIndex + 2];
							RawData[DestIndex + 3] = SrcData[SrcIndex + 3];
						}
					}
					else if(Bitmap.samplesPerPixel == 3)
					{
						for(size_t X = 0;X < Width;++X)
						{
							size_t SrcIndex = SRC_IMAGE_INDEX;
							size_t DestIndex = DEST_IMAGE_INDEX;

							RawData[DestIndex + 0] = SrcData[SrcIndex + 0];
							RawData[DestIndex + 1] = SrcData[SrcIndex + 1];
							RawData[DestIndex + 2] = SrcData[SrcIndex + 2];
							RawData[DestIndex + 3] = 0xff;
						}
					}
					else if(Bitmap.samplesPerPixel == 2)
					{
						// Placeholder - not sure if this is correct
						for(size_t X = 0;X < Width;++X)
						{
							size_t SrcIndex = SRC_IMAGE_INDEX;
							size_t DestIndex = DEST_IMAGE_INDEX;

							RawData[DestIndex + 0] = SrcData[SrcIndex + 0];
							RawData[DestIndex + 1] = SrcData[SrcIndex + 1];
							RawData[DestIndex + 2] = 0x00;
							RawData[DestIndex + 3] = 0xff;
						}
					}
					else if(Bitmap.samplesPerPixel == 1)
					{
						// Assume gray scale - copy single input channel to all color output channels
						for(size_t X = 0;X < Width;++X)
						{
							size_t SrcIndex = SRC_IMAGE_INDEX;
							size_t DestIndex = DEST_IMAGE_INDEX;

							RawData[DestIndex + 0] = SrcData[SrcIndex + 0];
							RawData[DestIndex + 1] = SrcData[SrcIndex + 0];
							RawData[DestIndex + 2] = SrcData[SrcIndex + 0];
							RawData[DestIndex + 3] = 0xff;
						}
					}
				}
				
				#undef SRC_IMAGE_INDEX
				#undef DEST_IMAGE_INDEX
			}
			else
			{
				// Unhandled case (e.g. Planar)
				SetError(TEXT("FIcnsImageWrapper: Cannot uncompress, unsupported format."));
				RawData.Reset();
			}

			if ((size_t)RawData.Num() >= DestImageSize && Format == ERGBFormat::BGRA)
			{
				for (size_t Index = 0; Index < DestImageSize; Index += 4)
				{
					uint8 Byte = RawData[Index];
					RawData[Index] = RawData[Index + 2];
					RawData[Index + 2] = Byte;
				}
			}
		}
		[Image release];
	}
#else
	checkf(false, TEXT("ICNS uncompressing not supported on this platform"));
#endif
}
