// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageWrapperBase.h"


/**
 * ICNS implementation of the helper class.
 */
class FIcnsImageWrapper
	: public FImageWrapperBase
{
public:

	/** Default Constructor. */
	FIcnsImageWrapper();

public:

	//~ FImageWrapper Interface

	virtual bool SetCompressed(const void* InCompressedData, int64 InCompressedSize) override;
	virtual bool SetRaw(const void* InRawData, int64 InRawSize, const int32 InWidth, const int32 InHeight, const ERGBFormat InFormat, const int32 InBitDepth, const int32 InBytesPerRow = 0) override;
	virtual void Compress(int32 Quality) override;
	virtual void Uncompress(const ERGBFormat InFormat, int32 InBitDepth) override;
	
	virtual bool CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const override;
	virtual ERawImageFormat::Type GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const override;
};
