// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formats/IcoImageWrapper.h"

#include "BmpImageSupport.h"
#include "Formats/PngImageWrapper.h"
#include "Formats/BmpImageWrapper.h"


#pragma pack(push,1)
struct FIconDirEntry
{
	uint8 bWidth;			// Width, in pixels, of the image
	uint8 bHeight;			// Height, in pixels, of the image
	uint8 bColorCount;		// Number of colors in image (0 if >=8bpp)
	uint8 bReserved;		// Reserved ( must be 0)
	uint16 wPlanes;			// Color Planes
	uint16 wBitCount;		// Bits per pixel
	uint32 dwBytesInRes;	// How many bytes in this resource?
	uint32 dwImageOffset;	// Where in the file is this image?
};
#pragma pack(pop)


#pragma pack(push,1)
struct FIconDir
{
	uint16 idReserved;			// Reserved (must be 0)
	uint16 idType;				// Resource Type (1 for icons)
	uint16 idCount;				// How many images?
	FIconDirEntry idEntries[1];	// An entry for each image (idCount of 'em)
};
#pragma pack(pop)


#pragma pack(push,1)
struct FRGBQuad
{
	uint8 rgbBlue;				// Blue channel
	uint8 rgbGreen;				// Green channel
	uint8 rgbRed;				// Red channel
	uint8 rgbReserved;			// Reserved (alpha)
};
#pragma pack(pop)


#pragma pack(push,1)
struct FIconImage
{
   FBitmapInfoHeader icHeader;		// DIB header
   FRGBQuad icColors[1];			// Color table
   uint8 icXOR[1];					// DIB bits for XOR mask
   uint8 icAND[1];					// DIB bits for AND mask
};
#pragma pack(pop)


/* FJpegImageWrapper structors
 *****************************************************************************/

FIcoImageWrapper::FIcoImageWrapper()
	: FImageWrapperBase()
{ }


/* FImageWrapper interface
 *****************************************************************************/
 
// CanSetRawFormat returns true if SetRaw will accept this format
bool FIcoImageWrapper::CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const
{
	//checkf(false, TEXT("ICO compression not supported"));
	return false;
}

// returns InFormat if supported, else maps to something supported
ERawImageFormat::Type FIcoImageWrapper::GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const
{
	//checkf(false, TEXT("ICO compression not supported"));
	return ERawImageFormat::BGRA8;
}

void FIcoImageWrapper::Compress( int32 Quality )
{
	checkf(false, TEXT("ICO compression not supported"));
}


void FIcoImageWrapper::Uncompress( const ERGBFormat InFormat, const int32 InBitDepth )
{
	const uint8* Buffer = CompressedData.GetData();

	if (ImageOffset != 0 && ImageSize != 0 && SubImageWrapper != nullptr)
	{
		SubImageWrapper->Uncompress(InFormat, InBitDepth);
		// Uncompress has no return value
		//  we can tell it failed if it set an error, or has no rawdata
		LastError = SubImageWrapper->GetLastError();
	}
}


bool FIcoImageWrapper::SetCompressed( const void* InCompressedData, int64 InCompressedSize )
{
	bool bResult = FImageWrapperBase::SetCompressed( InCompressedData, InCompressedSize );

	return bResult && LoadICOHeader();	// Fetch the variables from the header info
}


bool FIcoImageWrapper::GetRaw( const ERGBFormat InFormat, int32 InBitDepth, TArray64<uint8>& OutRawData )
{
	LastError.Empty();
	Uncompress(InFormat, InBitDepth);

	if (LastError.IsEmpty() && SubImageWrapper != nullptr)
	{
		SubImageWrapper->MoveRawData(OutRawData);
		if ( OutRawData.IsEmpty() )
		{
			return false;
		}
	}

	return LastError.IsEmpty();
}


/* FImageWrapper implementation
 *****************************************************************************/

bool FIcoImageWrapper::LoadICOHeader()
{
	const uint8* Buffer = CompressedData.GetData();
	uint64 BufferSize = CompressedData.Num();

#if WITH_UNREALPNG
	TSharedPtr<FPngImageWrapper> PngWrapper = MakeShareable(new FPngImageWrapper);
#endif
	TSharedPtr<FBmpImageWrapper> BmpWrapper = MakeShareable(new FBmpImageWrapper(false, true));

	bool bFoundImage = false;
	SubImageWrapper = nullptr;

	if ( BufferSize < sizeof(FIconDir) )
	{
		SetError(TEXT("LoadICOHeader corrupt; insufficient data"));
		return false;
	}
	const FIconDir* IconHeader = (FIconDir*)(Buffer);
	
	if (IconHeader->idReserved == 0 && IconHeader->idType == 1)
	{
		// use the largest-width 32-bit dir entry we find
		uint32 LargestWidth = 0;
		const FIconDirEntry* IconDirEntry = IconHeader->idEntries;
		
		if ( IconHeader->idCount == 0 )
		{
			SetError(TEXT("LoadICOHeader corrupt; no entries"));
			return false;
		}
		// FIconDir has 1 entry in it
		if ( BufferSize < sizeof(FIconDir) + (IconHeader->idCount-1)*sizeof(FIconDirEntry) )
		{
			SetError(TEXT("LoadICOHeader corrupt; insufficient data"));
			return false;
		}

		for (int32 Entry = 0; Entry < (int32)IconHeader->idCount; Entry++, IconDirEntry++)
		{
			const uint32 RealWidth = IconDirEntry->bWidth == 0 ? 256 : IconDirEntry->bWidth;
			if ( IconDirEntry->wBitCount == 32 && RealWidth > LargestWidth )
			{
				if ( (uint64)IconDirEntry->dwImageOffset + IconDirEntry->dwBytesInRes > BufferSize )
				{
					SetError(TEXT("LoadICOHeader corrupt; insufficient data"));
					return false;
				}

#if WITH_UNREALPNG
				if (PngWrapper->SetCompressed(Buffer + IconDirEntry->dwImageOffset, (int64)IconDirEntry->dwBytesInRes))
				{
					Width = PngWrapper->GetWidth();
					Height = PngWrapper->GetHeight();
					Format = PngWrapper->GetFormat();
					BitDepth = PngWrapper->GetBitDepth();
					LargestWidth = RealWidth;
					bFoundImage = true;
					bIsPng = true;
					ImageOffset = IconDirEntry->dwImageOffset;
					ImageSize = IconDirEntry->dwBytesInRes;
				}
				else 
#endif
				if (BmpWrapper->SetCompressed(Buffer + IconDirEntry->dwImageOffset, (int64)IconDirEntry->dwBytesInRes))
				{
					// otherwise this should be a BMP icon
					Width = BmpWrapper->GetWidth();
					Height = BmpWrapper->GetHeight();
					Format = BmpWrapper->GetFormat();
					BitDepth = BmpWrapper->GetBitDepth();
					LargestWidth = RealWidth;
					bFoundImage = true;
					bIsPng = false;
					ImageOffset = IconDirEntry->dwImageOffset;
					ImageSize = IconDirEntry->dwBytesInRes;
				}
			}
		}
	}

	if (bFoundImage)
	{
#if WITH_UNREALPNG
		if (bIsPng)
		{
			SubImageWrapper = PngWrapper;
		}
		else
#endif //WITH_UNREALPNG
		{
			SubImageWrapper = BmpWrapper;
		}
	}

	return bFoundImage;
}
