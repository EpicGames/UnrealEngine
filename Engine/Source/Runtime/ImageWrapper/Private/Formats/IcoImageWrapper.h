// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

#include "ImageWrapperBase.h"


/**
 * ICO implementation of the helper class.
 */
class FIcoImageWrapper
	: public FImageWrapperBase
{
public:

	/** Default Constructor. */
	FIcoImageWrapper();

public:

	//~ FImageWrapper Interface

	virtual void Compress(int32 Quality) override;
	virtual void Uncompress(const ERGBFormat InFormat, int32 InBitDepth) override;
	virtual bool SetCompressed(const void* InCompressedData, int64 InCompressedSize) override;
	virtual bool GetRaw(const ERGBFormat InFormat, int32 InBitDepth, TArray64<uint8>& OutRawData) override;
	
	virtual bool CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const override;
	virtual ERawImageFormat::Type GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const override;

protected:

	/** 
	 * Load the header information.
	 *
	 * @return true if successful
	 */
	bool LoadICOHeader();

private:

	/** Sub-wrapper component, as icons that contain PNG or BMP data */
	TSharedPtr<FImageWrapperBase> SubImageWrapper;

	/** Offset into file that we use as image data */
	uint32 ImageOffset;

	/** Size of image data in file */
	uint32 ImageSize;

	/** Whether we should use PNG or BMP data */
	bool bIsPng;
};
