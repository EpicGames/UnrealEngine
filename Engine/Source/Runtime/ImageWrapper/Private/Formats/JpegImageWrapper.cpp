// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formats/JpegImageWrapper.h"

#include "Math/Color.h"
#include "Misc/ScopeLock.h"
#include "ImageWrapperPrivate.h"
#include "ImageCoreUtils.h"

#if WITH_UNREALJPEG

#ifdef __clang__
	#pragma clang diagnostic push
	#pragma clang diagnostic ignored "-Wshadow"

	#if PLATFORM_UNIX || PLATFORM_MAC
		#pragma clang diagnostic ignored "-Wshift-negative-value"	// clang 3.7.0
	#endif
#endif


#if WITH_LIBJPEGTURBO
	THIRD_PARTY_INCLUDES_START
	#pragma push_macro("DLLEXPORT")
	#undef DLLEXPORT // libjpeg-turbo defines DLLEXPORT as well
	#include "turbojpeg.h"
	#pragma pop_macro("DLLEXPORT")
	THIRD_PARTY_INCLUDES_END
#else
	PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
	#include "jpgd.h"
	#include "jpgd.cpp"
	#include "jpge.h"
	#include "jpge.cpp"
	PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS
#endif	// WITH_LIBJPEGTURBO

#ifdef __clang__
	#pragma clang diagnostic pop
#endif

#if WITH_LIBJPEGTURBO
namespace
{
	int ConvertTJpegPixelFormat(ERGBFormat InFormat)
	{
		// note: libjpeg-turbo currently does not actually read/write A
		//	TJPF_BGRA is a synonym for TJPF_BGRX
		switch (InFormat)
		{
			case ERGBFormat::BGRA:	return TJPF_BGRA;
			case ERGBFormat::Gray:	return TJPF_GRAY;
			case ERGBFormat::RGBA:	return TJPF_RGBA;
			default:	check(0);	return TJPF_RGBA;
		}
	}
}
#endif	// WITH_LIBJPEGTURBO


#if WITH_LIBJPEGTURBO
// libjpeg-turbo since version 2.0.5 is thread safe
#define JPEG_NEEDS_CRITSEC 0
#else
// legacy libjpeg is not thread safe
#define JPEG_NEEDS_CRITSEC 1
#endif

#if JPEG_NEEDS_CRITSEC
// Only allow one thread to use JPEG decoder at a time (it's not thread safe)
static FCriticalSection GJPEGSection; 

#define JPEG_SCOPE_CRITSEC()	FScopeLock JPEGLock(&GJPEGSection)
#else
#define JPEG_SCOPE_CRITSEC()	do { } while(0)
#endif

/* FJpegImageWrapper structors
 *****************************************************************************/

FJpegImageWrapper::FJpegImageWrapper(int32 InNumComponents /* = 4 */)
	: FImageWrapperBase()
	, NumComponents(InNumComponents)
#if WITH_LIBJPEGTURBO
	, Decompressor(0)
#endif	// WITH_LIBJPEGTURBO
{ 
	// NumComponents == 1 for GrayscaleJPEG
}


FJpegImageWrapper::~FJpegImageWrapper()
{
	Reset();
}

void FJpegImageWrapper::Reset()
{
	FImageWrapperBase::Reset();

#if WITH_LIBJPEGTURBO
	if (Decompressor)
	{
		JPEG_SCOPE_CRITSEC();

		tjDestroy(Decompressor);
		Decompressor = 0;
	}
#endif

}

/* FImageWrapperBase interface
 *****************************************************************************/

bool FJpegImageWrapper::SetCompressed(const void* InCompressedData, int64 InCompressedSize)
{
#if WITH_LIBJPEGTURBO
	return SetCompressedTurbo(InCompressedData, InCompressedSize);
#else
	// jpgd doesn't support 64-bit sizes.
	if (InCompressedSize < 0 || InCompressedSize > MAX_uint32)
	{
		return false;
	}

	jpgd::jpeg_decoder_mem_stream jpeg_memStream((uint8*)InCompressedData, (uint32)InCompressedSize);

	jpgd::jpeg_decoder decoder(&jpeg_memStream);
	if (decoder.get_error_code() != jpgd::JPGD_SUCCESS)
	{
		return false;
	}

	if ( ! FImageWrapperBase::SetCompressed(InCompressedData, InCompressedSize) )
	{
		return false;
	}

	// We don't support 16 bit jpegs
	BitDepth = 8;

	Width = decoder.get_width();
	Height = decoder.get_height();

	switch (decoder.get_num_components())
	{
	case 1:
		Format = ERGBFormat::Gray;
		break;
	case 3:
		Format = ERGBFormat::RGBA;
		break;
	default:
		return false;
	}
	
	if ( ! FImageCoreUtils::IsImageImportPossible(Width,Height) )
	{
		SetError(TEXT("Image dimensions are not possible to import"));
		return false;
	}

	return true;
#endif
}


// CanSetRawFormat returns true if SetRaw will accept this format
bool FJpegImageWrapper::CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const
{
	return ((InFormat == ERGBFormat::RGBA || InFormat == ERGBFormat::BGRA || InFormat == ERGBFormat::Gray) && InBitDepth == 8);
}

// returns InFormat if supported, else maps to something supported
ERawImageFormat::Type FJpegImageWrapper::GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const
{
	switch(InFormat)
	{
	case ERawImageFormat::G8:
	case ERawImageFormat::BGRA8:
		return InFormat; // directly supported
	case ERawImageFormat::BGRE8:
	case ERawImageFormat::RGBA16:
	case ERawImageFormat::RGBA16F:
	case ERawImageFormat::RGBA32F:
	case ERawImageFormat::G16:
	case ERawImageFormat::R16F:
	case ERawImageFormat::R32F:
		return ERawImageFormat::BGRA8; // needs conversion
	default:
		check(0);
		return ERawImageFormat::BGRA8;
	};
}

void FJpegImageWrapper::Compress(int32 Quality)
{
	if (Quality == 0)
	{
		//default 
		Quality = 85;
	}
	else if (Quality == (int32)EImageCompressionQuality::Uncompressed)
	{
		// fix = 1 (Uncompressed) was previously treated as max-compress
		Quality = 100;
	}
	else
	{
		ensure(Quality >= 1 && Quality <= 100);

		#define JPEG_QUALITY_MIN 40
		// JPEG should not be used below quality JPEG_QUALITY_MIN
		Quality = FMath::Clamp(Quality, JPEG_QUALITY_MIN, 100);
	}

#if WITH_LIBJPEGTURBO
	CompressTurbo(Quality);
#else
	if (CompressedData.Num() == 0)
	{
		JPEG_SCOPE_CRITSEC();
		
		check(RawData.Num());
		check(Width > 0);
		check(Height > 0);

		// re-order components if required - JPEGs expect RGBA
		if(Format == ERGBFormat::BGRA)
		{
			FColor* Colors = (FColor*)RawData.GetData();
			const int32 NumColors = RawData.Num() / 4;
			for(int32 ColorIndex = 0; ColorIndex < NumColors; ColorIndex++)
			{
				Colors[ColorIndex].B = Colors[ColorIndex].B ^ Colors[ColorIndex].R;
				Colors[ColorIndex].R = Colors[ColorIndex].R ^ Colors[ColorIndex].B;
				Colors[ColorIndex].B = Colors[ColorIndex].B ^ Colors[ColorIndex].R;
			}
		}


		// Note: OutBufferSize intentionally uses int64_t type as that's what jpge::compress_image_to_jpeg_file_in_memory expects.
		// UE int64 type is not compatible with int64_t on all compilers (int64_t may be `long`, while int64 is `long long`).
		// jpge::compress_image_to_jpeg_file_in_memory recommends at least 1kb buffer size since small images
		// have some overhead and certain configurations can expand in size (e.g. high quality)
		int64_t OutBufferSize = std::max<int64>(RawData.Num(), 1024);
		
		CompressedData.Reset(OutBufferSize);
		CompressedData.AddUninitialized(OutBufferSize);

		jpge::params Parameters;
		Parameters.m_quality = Quality;
		bool bSuccess = jpge::compress_image_to_jpeg_file_in_memory(
			CompressedData.GetData(), OutBufferSize, Width, Height, NumComponents, RawData.GetData(), Parameters);
		
		check(bSuccess);

		CompressedData.RemoveAt((int64)OutBufferSize, CompressedData.Num() - (int64)OutBufferSize);
	}
#endif
}


void FJpegImageWrapper::Uncompress(const ERGBFormat InFormat, int32 InBitDepth)
{
#if WITH_LIBJPEGTURBO
	UncompressTurbo(InFormat, InBitDepth);
#else
	// Ensure we haven't already uncompressed the file.
	if (RawData.Num() != 0)
	{
		return;
	}

	// Get the number of channels we need to extract
	int Channels = 0;
	if ((InFormat == ERGBFormat::RGBA || InFormat == ERGBFormat::BGRA) && InBitDepth == 8)
	{
		Channels = 4;
	}
	else if (InFormat == ERGBFormat::Gray && InBitDepth == 8)
	{
		Channels = 1;
	}
	else
	{
		check(false);
		return;
	}

	JPEG_SCOPE_CRITSEC();

	check(CompressedData.Num());

	int32 NumColors;
	int32 jpegWidth,jpegHeight;
	uint8* OutData = jpgd::decompress_jpeg_image_from_memory(
		CompressedData.GetData(), CompressedData.Num(), &jpegWidth, &jpegHeight, &NumColors, Channels, (int)InFormat);

	if (OutData)
	{
		Width = jpegWidth;
		Height = jpegHeight;
		RawData.Reset(Width * Height * Channels);
		RawData.AddUninitialized(Width * Height * Channels);
		FMemory::Memcpy(RawData.GetData(), OutData, RawData.Num());
		FMemory::Free(OutData);
	}
	else
	{
		UE_LOGF(LogImageWrapper, Error, "JPEG Decompress Error");
		RawData.Empty();
	}
#endif
}

#if WITH_LIBJPEGTURBO
bool FJpegImageWrapper::SetCompressedTurbo(const void* InCompressedData, int64 InCompressedSize)
{
	// SetCompressed does Reset
	if ( ! FImageWrapperBase::SetCompressed(InCompressedData, InCompressedSize) )
	{
		return false;
	}
	
	JPEG_SCOPE_CRITSEC();

	check( Decompressor == 0 );
	Decompressor = tjInitDecompress();

	int ImageWidth;
	int ImageHeight;
	int SubSampling;
	int ColorSpace;
	if (tjDecompressHeader3(Decompressor, reinterpret_cast<const uint8*>(InCompressedData), InCompressedSize, &ImageWidth, &ImageHeight, &SubSampling, &ColorSpace) != 0)
	{
		SetError(TEXT("tjDecompressHeader3 failed"));
		return false;
	}

	if (ColorSpace == TJCS_CMYK || ColorSpace == TJCS_YCCK)
	{
		UE_LOGF(LogImageWrapper, Error, "Unsupported CMYK colorspace for JPEG, only Grayscale and RGB are supported");
		return false;
	}

	// set after call to base SetCompressed as it will reset members
	Width = ImageWidth;
	Height = ImageHeight;
	BitDepth = 8; // We don't support 16 bit jpegs

	// if NumComponents == 1 (for GrayscaleJPEG format), force ERGBFormat::Gray ?
	Format = ( SubSampling == TJSAMP_GRAY ) ? ERGBFormat::Gray : ERGBFormat::BGRA;
	
	if ( ! FImageCoreUtils::IsImageImportPossible(Width,Height) )
	{
		SetError(TEXT("Image dimensions are not possible to import"));
		return false;
	}

	// Decompressor is retained until Uncompress

	return true;
}

void FJpegImageWrapper::CompressTurbo(int32 Quality)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FJpegImageWrapper::CompressTurbo);

	if (CompressedData.Num() == 0)
	{
		JPEG_SCOPE_CRITSEC();

		// Quality mapping should have already been done
		check( Quality >= JPEG_QUALITY_MIN && Quality <= 100 );

		check(RawData.Num());
		check(Width > 0);
		check(Height > 0);
		check(BitDepth == 8);

		const int PixelFormat = ConvertTJpegPixelFormat(Format);

		// NumComponents == 1 for GrayscaleJPEG
		const int Subsampling = (NumComponents == 1 || Format == ERGBFormat::Gray) ? TJSAMP_GRAY : TJSAMP_420;
		const int Flags = TJFLAG_NOREALLOC | TJFLAG_FASTDCT;

		unsigned long OutBufferSize = tjBufSize(Width, Height, Subsampling);
		CompressedData.SetNum(OutBufferSize);
		unsigned char* OutBuffer = CompressedData.GetData();

		int BytesPerRow = GetBytesPerRow();
		
		tjhandle Compressor = tjInitCompress();

		const bool bSuccess = tjCompress2(Compressor, RawData.GetData(), Width, BytesPerRow, Height, PixelFormat, &OutBuffer, &OutBufferSize, Subsampling, Quality, Flags) == 0;
		check(bSuccess);
		
		tjDestroy(Compressor);

		if ( ! bSuccess )
		{
			CompressedData.Empty();
			SetError(TEXT("tjCompress2 failed"));
		}
		else
		{
			check( CompressedData.GetData() == OutBuffer ); // TJFLAG_NOREALLOC so OutBuffer should not have changed
			CompressedData.SetNum(OutBufferSize);
		}
	}
}

void FJpegImageWrapper::UncompressTurbo(const ERGBFormat InFormat, int32 InBitDepth)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FJpegImageWrapper::UncompressTurbo);

	// Ensure we haven't already uncompressed the file.
	if (RawData.Num() != 0)
	{
		return;
	}

	// Get the number of channels we need to extract
	int Channels = 0;
	if ((InFormat == ERGBFormat::RGBA || InFormat == ERGBFormat::BGRA) && InBitDepth == 8)
	{
		Channels = 4;
	}
	else if (InFormat == ERGBFormat::Gray && InBitDepth == 8)
	{
		Channels = 1;
	}
	else
	{
		check(false);
		return;
	}

	JPEG_SCOPE_CRITSEC();

	check(Decompressor);
	check(CompressedData.Num());

	RawData.Reset(Width * Height * Channels);
	RawData.AddUninitialized(Width * Height * Channels);
	const int PixelFormat = ConvertTJpegPixelFormat(InFormat);
	const int Flags = TJFLAG_NOREALLOC | TJFLAG_FASTDCT;
	// @todo Oodle : evaluate whether TJFLAG_FASTDCT quality loss is worth the speed gained

	if (tjDecompress2(Decompressor, CompressedData.GetData(), CompressedData.Num(), RawData.GetData(), Width, 0, Height, PixelFormat, Flags) != 0)
	{
		UE_LOGF(LogImageWrapper, Error, "JPEG Decompress Error - %ls", ANSI_TO_TCHAR(tj3GetErrorStr(Decompressor)));
		SetError(TEXT("tjDecompress2 failed"));
		RawData.Empty();
		return;
	}
}
#endif	// WITH_LIBJPEGTURBO


#endif //WITH_JPEG
