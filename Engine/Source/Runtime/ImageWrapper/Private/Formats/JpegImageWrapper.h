// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ImageWrapperBase.h"

#if WITH_UNREALJPEG

#if WITH_LIBJPEGTURBO
using tjhandle = void*;
#endif	// WITH_LIBJPEGTURBO

/**
 * Uncompresses JPEG data to raw 24bit RGB image that can be used by Unreal textures.
 *
 * Source code for JPEG decompression from http://code.google.com/p/jpeg-compressor/
 */
class FJpegImageWrapper
	: public FImageWrapperBase
{
public:

	/** Default constructor. */
	// NumComponents == 1 for GrayscaleJPEG , but not supported by TJPEG path so currently broken
	FJpegImageWrapper(int32 InNumComponents = 4);

	virtual ~FJpegImageWrapper();
	virtual void Reset() override;

public:

	//~ FImageWrapperBase interface

	virtual bool SetCompressed(const void* InCompressedData, int64 InCompressedSize) override;
	virtual void Uncompress(const ERGBFormat InFormat, int32 InBitDepth) override;
	virtual void Compress(int32 Quality) override;
	
	virtual bool CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const override;
	virtual ERawImageFormat::Type GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const override;

private:

	int32 NumComponents;
	
#if WITH_LIBJPEGTURBO
	bool SetCompressedTurbo(const void* InCompressedData, int64 InCompressedSize);
	void CompressTurbo(int32 Quality);
	void UncompressTurbo(const ERGBFormat InFormat, int32 InBitDepth);
#endif	// WITH_LIBJPEGTURBO

#if WITH_LIBJPEGTURBO
	tjhandle Decompressor;
#endif	// WITH_LIBJPEGTURBO
};


#endif //WITH_JPEG
