// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formats/PngImageWrapper.h"
#include "ImageWrapperPrivate.h"

#include "Containers/Utf8String.h"
#include "Math/GuardedInt.h"
#include "Misc/ScopeLock.h"
#include "ImageCoreUtils.h"

#if WITH_UNREALPNG

// Disable warning "interaction between '_setjmp' and C++ object destruction is non-portable"
#ifdef _MSC_VER
	#pragma warning(push)
	#pragma warning(disable:4611)
#endif

#if PLATFORM_ANDROID
/** Only allow one thread to use libpng at a time (it's not thread safe) */
static FCriticalSection GPNGSection;
#endif

/* Local helper classes
 *****************************************************************************/

/**
 * Error type for PNG reading issue.
 */
struct FPNGImageCRCError
{
	FPNGImageCRCError(FString InErrorText)
		: ErrorText(MoveTemp(InErrorText))
	{ }

	FString ErrorText;
};


/**
 * Guard that safely releases PNG reader resources.
 */
class PNGReadGuard
{
public:
	PNGReadGuard(png_structp* InReadStruct, png_infop* InInfo)
		: png_ptr(InReadStruct)
		, info_ptr(InInfo)
		, PNGRowPointers(NULL)
	{
	}

	~PNGReadGuard()
	{
		if (PNGRowPointers != NULL)
		{
			png_free(*png_ptr, PNGRowPointers);
		}
		png_destroy_read_struct(png_ptr, info_ptr, NULL);
	}

	void SetRowPointers(png_bytep* InRowPointers)
	{
		PNGRowPointers = InRowPointers;
	}

private:
	png_structp* png_ptr;
	png_infop* info_ptr;
	png_bytep* PNGRowPointers;
};


/**
 * Guard that safely releases PNG Writer resources
 */
class PNGWriteGuard
{
public:

	PNGWriteGuard(png_structp* InWriteStruct, png_infop* InInfo)
		: PNGWriteStruct(InWriteStruct)
		, info_ptr(InInfo)
		, PNGRowPointers(NULL)
	{
	}

	~PNGWriteGuard()
	{
		if (PNGRowPointers != NULL)
		{
			png_free(*PNGWriteStruct, PNGRowPointers);
		}

		png_destroy_write_struct(PNGWriteStruct, info_ptr);
	}

	void SetRowPointers(png_bytep* InRowPointers)
	{
		PNGRowPointers = InRowPointers;
	}

	TArray<png_text> MetadataKeyValuePairs;
	TArray<FUtf8String> MetadataStrings;

private:
	png_structp* PNGWriteStruct;
	png_infop* info_ptr;
	png_bytep* PNGRowPointers;
};


/* FPngImageWrapper structors
 *****************************************************************************/

FPngImageWrapper::FPngImageWrapper()
	: FImageWrapperBase()
	, ReadOffset(0)
	, ColorType(0)
	, Channels(0)
{ }


/* FImageWrapper interface
 *****************************************************************************/
 
// CanSetRawFormat returns true if SetRaw will accept this format
bool FPngImageWrapper::CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const
{
	return (InFormat == ERGBFormat::RGBA || InFormat == ERGBFormat::BGRA || InFormat == ERGBFormat::Gray) && 
		( InBitDepth == 8 || InBitDepth == 16 );
}

// returns InFormat if supported, else maps to something supported
ERawImageFormat::Type FPngImageWrapper::GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const
{
	switch(InFormat)
	{
	case ERawImageFormat::G8:
	case ERawImageFormat::BGRA8:
	case ERawImageFormat::RGBA16:
	case ERawImageFormat::G16:
		return InFormat; // directly supported
	case ERawImageFormat::BGRE8:
	case ERawImageFormat::RGBA16F:
	case ERawImageFormat::RGBA32F:
	case ERawImageFormat::R16F:
	case ERawImageFormat::R32F:
		return ERawImageFormat::RGBA16; // needs conversion
	default:
		check(0);
		return ERawImageFormat::BGRA8;
	};
}

void FPngImageWrapper::Compress(int32 Quality)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPngImageWrapper::Compress)

	if (!CompressedData.Num())
	{
		//Preserve old single thread code on some platform in relation to a type incompatibility at compile time.
#if PLATFORM_ANDROID
		// thread safety
		FScopeLock PNGLock(&GPNGSection);
#endif

		check(RawData.Num());
		check(Width > 0);
		check(Height > 0);

		// Reset to the beginning of file so we can use png_read_png(), which expects to start at the beginning.
		ReadOffset = 0;
		
		png_structp png_ptr	= png_create_write_struct_2(PNG_LIBPNG_VER_STRING, this, FPngImageWrapper::user_error_fn, FPngImageWrapper::user_warning_fn, NULL, FPngImageWrapper::user_malloc, FPngImageWrapper::user_free);
		check(png_ptr);

		png_infop info_ptr	= png_create_info_struct(png_ptr);
		check(info_ptr);

		png_bytep* row_pointers = (png_bytep*) png_malloc( png_ptr, Height*sizeof(png_bytep) );
		
		PNGWriteGuard PNGGuard(&png_ptr, &info_ptr);
		PNGGuard.SetRowPointers( row_pointers );

		// Store the current stack pointer in the jump buffer. setjmp will return non-zero in the case of a write error.
#if PLATFORM_ANDROID
		//Preserve old single thread code on some platform in relation to a type incompatibility at compile time.
		if (setjmp(SetjmpBuffer) != 0)
#else
		//Use libPNG jump buffer solution to allow concurrent compression\decompression on concurrent threads.
		if (setjmp(png_jmpbuf(png_ptr)) != 0)
#endif
		{
			CompressedData.Empty();
			UE_LOGF(LogImageWrapper, Error, "PNG Compress Error");
			return;
		}

		// ---------------------------------------------------------------------------------------------------------
		// Anything allocated on the stack after this point will not be destructed correctly in the case of an error

		{
			int ZlibLevel = 3; // default
			// Quality == 0 is the default argument, does not set a zlib level
			if ( Quality != 0 )
			{
				if ( Quality == (int32)EImageCompressionQuality::Uncompressed )
				{
					ZlibLevel = 0; // compression off
				}
				else if ( -Quality >= 1 && -Quality <= 9 )
				{
					// negative quality for zlib level
					ZlibLevel = -Quality;
				}
				else if ( Quality >= 20 && Quality <= 100 )
				{
					// JPEG quality, just ignore
					// calls to GetCompressed(100) are common
				}
				else
				{
					UE_LOGF(LogImageWrapper, Warning, "PNG Quality ZlibLevel out of range %d", Quality );
				}
			}

			PNGGuard.MetadataKeyValuePairs.Reserve(TextBlocks.Num());
			PNGGuard.MetadataStrings.Reserve(TextBlocks.Num() * 2);

			for (auto& Block : TextBlocks)
			{
				png_text info {};
				info.compression = PNG_TEXT_COMPRESSION_NONE;

				PNGGuard.MetadataStrings.Emplace((*Block.Key));
				info.key = (char*)*PNGGuard.MetadataStrings.Last();

				PNGGuard.MetadataStrings.Emplace((*Block.Value));
				info.text = (char*)*PNGGuard.MetadataStrings.Last();

				PNGGuard.MetadataKeyValuePairs.Add(info);
			}

			png_set_text(png_ptr, info_ptr, PNGGuard.MetadataKeyValuePairs.GetData(), PNGGuard.MetadataKeyValuePairs.Num());

			png_set_compression_level(png_ptr, ZlibLevel);
			png_set_IHDR(png_ptr, info_ptr, Width, Height, BitDepth, (Format == ERGBFormat::Gray) ? PNG_COLOR_TYPE_GRAY : PNG_COLOR_TYPE_RGBA, PNG_INTERLACE_NONE, PNG_COMPRESSION_TYPE_DEFAULT, PNG_FILTER_TYPE_DEFAULT);
			png_set_write_fn(png_ptr, this, FPngImageWrapper::user_write_compressed, FPngImageWrapper::user_flush_data);

			// If we're writing an uncompressed PNG, then we're expecting to be compressing
			// externally and we can assume we're interested in speed. In this case, we force
			// the fastest filter to avoid things like Paeth. This is ~2x speedup in decompressing
			// bulk data encoded in this manner.
			if (Quality == (int32)EImageCompressionQuality::Uncompressed)
			{
				png_set_filter(png_ptr, 0, PNG_FILTER_UP | PNG_FILTER_VALUE_UP);
			}

			const uint64 PixelChannels = (Format == ERGBFormat::Gray) ? 1 : 4;
			const uint64 BytesPerPixel = (BitDepth * PixelChannels) / 8;
			const uint64 BytesPerRow = BytesPerPixel * Width;

			for (int64 i = 0; i < Height; i++)
			{
				row_pointers[i]= &RawData[i * BytesPerRow];
			}
			png_set_rows(png_ptr, info_ptr, row_pointers);

			uint32 Transform = (Format == ERGBFormat::BGRA) ? PNG_TRANSFORM_BGR : PNG_TRANSFORM_IDENTITY;

			// PNG files store 16-bit pixels in network byte order (big-endian, ie. most significant bits first).
#if PLATFORM_LITTLE_ENDIAN
			// We're little endian so we need to swap
			if (BitDepth == 16)
			{
				Transform |= PNG_TRANSFORM_SWAP_ENDIAN;
			}
#endif

			png_write_png(png_ptr, info_ptr, Transform, NULL);
		}
	}
}


void FPngImageWrapper::Reset()
{
	FImageWrapperBase::Reset();

	ReadOffset = 0;
	ColorType = 0;
	Channels = 0;
}


bool FPngImageWrapper::SetCompressed(const void* InCompressedData, int64 InCompressedSize)
{
	if ( ! FImageWrapperBase::SetCompressed(InCompressedData, InCompressedSize) )
	{
		return false;
	}

	if ( ! LoadPNGHeader() )
	{
		return false;
	}
	
	if ((BitDepth == 1 || BitDepth == 2 || BitDepth == 4) && ((ColorType & PNG_COLOR_MASK_ALPHA) == 0))
	{
		// PNG specfication:
		//  (http://www.libpng.org/pub/png/spec/1.2/PNG-Chunks.html)
		if ((ColorType == PNG_COLOR_TYPE_PALETTE) || (ColorType == PNG_COLOR_TYPE_GRAY))
		{
			//From png specification:
			//  Note that the palette uses 8 bits (1 byte) per sample regardless of the image bit depth specification. 
			//  In particular, the palette is 8 bits deep even when it is a suggested quantization of a 16-bit truecolor image.

			// ColorType == PNG_COLOR_TYPE_PALETTE supported via:
			//	png_set_palette_to_rgb (called in UncompressPNGData)

			// ColorType == PNG_COLOR_TYPE_GRAYsupported via:
			//	png_set_expand_gray_1_2_4_to_8 (called in UncompressPNGData)

			if (!RawData.Num())
			{
				check(CompressedData.Num());
				UncompressPNGData(Format, 8);

				// after UncompressPNGData , BitDepth is now changed to 8
			}
		}
		else if (ColorType == PNG_COLOR_TYPE_RGB)
		{
			//according to png specification this is not a possiblity
		}
	}
	
	if ( (Format == ERGBFormat::BGRA || Format == ERGBFormat::RGBA || Format == ERGBFormat::Gray) &&
		(BitDepth == 8 || BitDepth == 16) )
	{
	
		if ( ! FImageCoreUtils::IsImageImportPossible(Width,Height) )
		{
			SetError(TEXT("Image dimensions are not possible to import"));
			return false;
		}

		return true;
	}
	else
	{
		// Other formats unsupported at present
		UE_LOGF(LogImageWrapper, Warning, "PNG Unsupported Format");
		return false;
	}
}


void FPngImageWrapper::Uncompress(const ERGBFormat InFormat, const int32 InBitDepth)
{
	if(!RawData.Num() || InFormat != Format || InBitDepth != BitDepth)
	{
		check(CompressedData.Num());
		UncompressPNGData(InFormat, InBitDepth);
	}
}


void FPngImageWrapper::UncompressPNGData(const ERGBFormat InFormat, const int32 InBitDepth)
{
	//Preserve old single thread code on some platform in relation to a type incompatibility at compile time.
#if PLATFORM_ANDROID
	// thread safety
	FScopeLock PNGLock(&GPNGSection);
#endif

	check(CompressedData.Num());
	check(Width > 0);
	check(Height > 0);

	// Note that PNGs on PC tend to be BGR
	check(InFormat == ERGBFormat::BGRA || InFormat == ERGBFormat::RGBA || InFormat == ERGBFormat::Gray);	// Other formats unsupported at present
	check(InBitDepth == 1 || InBitDepth == 2 || InBitDepth == 4 || InBitDepth == 8 || InBitDepth == 16);	// Other formats unsupported at present

	// Check if LibPNG can handle the size
	if (Height > PNG_UINT_32_MAX / sizeof(png_bytep))
	{
		RawData.Empty();
		UE_LOGF(LogImageWrapper, Error, "PNG image too high to process with libPNG");
		return;
	}

	// Reset to the beginning of file so we can use png_read_png(), which expects to start at the beginning.
	ReadOffset = 0;
		
	png_structp png_ptr	= png_create_read_struct_2(PNG_LIBPNG_VER_STRING, this, FPngImageWrapper::user_error_fn, FPngImageWrapper::user_warning_fn, NULL, FPngImageWrapper::user_malloc, FPngImageWrapper::user_free);
	check(png_ptr);

	png_infop info_ptr	= png_create_info_struct(png_ptr);
	check(info_ptr);

#if !PLATFORM_EXCEPTIONS_DISABLED
	try
#endif
	{
		png_bytep* row_pointers = (png_bytep*) png_malloc( png_ptr, Height*sizeof(png_bytep) );
		PNGReadGuard PNGGuard( &png_ptr, &info_ptr );
		PNGGuard.SetRowPointers(row_pointers);

		// Store the current stack pointer in the jump buffer. setjmp will return non-zero in the case of a read error.
#if PLATFORM_ANDROID
		//Preserve old single thread code on some platform in relation to a type incompatibility at compile time.
		if (setjmp(SetjmpBuffer) != 0)
#else
		//Use libPNG jump buffer solution to allow concurrent compression\decompression on concurrent threads.
		if (setjmp(png_jmpbuf(png_ptr)) != 0)
#endif
		{
			RawData.Empty();
			UE_LOGF(LogImageWrapper, Error, "PNG Decompress Error");
			return;
		}

		// ---------------------------------------------------------------------------------------------------------
		// Anything allocated on the stack after this point will not be destructed correctly in the case of an error
		{
			png_set_read_fn(png_ptr, this, FPngImageWrapper::user_read_compressed);

			// Read the header
			png_read_info(png_ptr, info_ptr);

			// We want paletted data expanded out to RGB and transparencies turned into alpha
			png_set_expand(png_ptr);

			// Insert alpha channel with full opacity for RGB images without alpha
			if ((ColorType & PNG_COLOR_MASK_ALPHA) == 0 && (InFormat == ERGBFormat::BGRA || InFormat == ERGBFormat::RGBA))
			{
				if (InBitDepth == 8)
				{
					png_set_add_alpha(png_ptr, 0xff , PNG_FILLER_AFTER);
				}
				else if (InBitDepth == 16)
				{
					png_set_add_alpha(png_ptr, 0xffff , PNG_FILLER_AFTER);
				}
			}

			// Calculate Pixel Depth
			const uint64 PixelChannels = (InFormat == ERGBFormat::Gray) ? 1 : 4;
			const uint64 BytesPerPixel = (InBitDepth * PixelChannels) / 8;
			check(BytesPerPixel > 0);

			const int64 BytesPerRow = BytesPerPixel * Width;
			check(Height <= MAX_int64 / BytesPerRow);	// check for overflow on multiplication
			const int64 TotalBytes = Height * BytesPerRow;

			RawData.Empty(TotalBytes);
			RawData.AddUninitialized(TotalBytes);

			for (int64 i = 0; i < Height; i++)
			{
				row_pointers[i]= &RawData[i * BytesPerRow];
			}
			png_set_rows(png_ptr, info_ptr, row_pointers);

			// PNG files are RGBA, if we want BGRA we need to swap
			if (InFormat == ERGBFormat::BGRA)
			{
				png_set_bgr(png_ptr);
			}
			
			// PNG files store 16-bit pixels in network byte order (big-endian, ie. most significant bits first).
#if PLATFORM_LITTLE_ENDIAN
			// We're little endian so we need to swap
			if (BitDepth == 16)
			{
				png_set_swap(png_ptr);
			}
#endif

			// @todo Oodle : remove all these conversions
			//	ImageWrappers should load images as they are in the file
			//	FImage does conversions after loading

			// Convert grayscale png to RGB if requested
			if ((ColorType & PNG_COLOR_MASK_COLOR) == 0 &&
				(InFormat == ERGBFormat::RGBA || InFormat == ERGBFormat::BGRA))
			{
				png_set_gray_to_rgb(png_ptr);
			}

			// Convert RGB png to grayscale if requested
			if ((ColorType & PNG_COLOR_MASK_COLOR) != 0 && InFormat == ERGBFormat::Gray)
			{
				png_set_rgb_to_gray_fixed(png_ptr, PNG_ERROR_ACTION_WARN, -1, -1);
			}

			// Strip alpha channel if requested output is grayscale
			if (InFormat == ERGBFormat::Gray)
			{
				// this is not necessarily the best option, instead perhaps:
				// png_color background = {0,0,0};
				// png_set_background(png_ptr, &background, PNG_BACKGROUND_GAMMA_SCREEN, 0, 1.0);
				png_set_strip_alpha(png_ptr);
			}

			// Reduce 16-bit to 8-bit if requested
			if (BitDepth == 16 && InBitDepth == 8)
			{
				png_set_scale_16(png_ptr);
			}

			// Increase 8-bit to 16-bit if requested
			if (BitDepth <= 8 && InBitDepth == 16)
			{
				png_set_expand_16(png_ptr);
			}

			// Finalize init
			(void)png_set_interlace_handling(png_ptr);
			png_read_update_info(png_ptr, info_ptr);

			// Read the PNG
			png_read_image(png_ptr, row_pointers);

			png_read_end(png_ptr, info_ptr);
		}
	}
#if !PLATFORM_EXCEPTIONS_DISABLED
	catch (const FPNGImageCRCError& e)
	{
		/** 
		 *	libPNG has a known issue in version 1.5.2 causing
		 *	an unhandled exception upon a CRC error. This code 
		 *	catches our custom exception thrown in user_error_fn.
		 */
		UE_LOGF(LogImageWrapper, Error, "FPNGImageCRCError: %ls", *e.ErrorText);
	}
#endif

	Format = InFormat;
	BitDepth = InBitDepth;
}


/* FPngImageWrapper implementation
 *****************************************************************************/


bool FPngImageWrapper::IsPNG() const
{
	check(CompressedData.Num());

	const int32 PNGSigSize = sizeof(png_size_t);

	if (CompressedData.Num() > PNGSigSize)
	{
		png_size_t PNGSignature = *reinterpret_cast<const png_size_t*>(CompressedData.GetData());
		return (0 == png_sig_cmp(reinterpret_cast<png_bytep>(&PNGSignature), 0, PNGSigSize));
	}

	return false;
}

bool FPngImageWrapper::LoadPNGHeader()
{
	check(CompressedData.Num());

	// Test whether the data this PNGLoader is pointing at is a PNG or not.
	if (IsPNG())
	{
#if PLATFORM_ANDROID
		// thread safety
		FScopeLock PNGLock(&GPNGSection);
#endif

		png_structp png_ptr = png_create_read_struct_2(PNG_LIBPNG_VER_STRING, this, FPngImageWrapper::user_error_fn, FPngImageWrapper::user_warning_fn, NULL, FPngImageWrapper::user_malloc, FPngImageWrapper::user_free);
		check(png_ptr);

		png_infop info_ptr = png_create_info_struct(png_ptr);
		check(info_ptr);

		PNGReadGuard PNGGuard(&png_ptr, &info_ptr);

		// Store the current stack pointer in the jump buffer. setjmp will return non-zero in the case of a read error.
#if PLATFORM_ANDROID
		//Preserve old single thread code on some platform in relation to a type incompatibility at compile time.
		if (setjmp(SetjmpBuffer) != 0)
#else
		//Use libPNG jump buffer solution to allow concurrent compression\decompression on concurrent threads.
		if (setjmp(png_jmpbuf(png_ptr)) != 0)
#endif
		{
			UE_LOGF(LogImageWrapper, Error, "PNG Header Error");
			return false;
		}

		// ---------------------------------------------------------------------------------------------------------
		// Anything allocated on the stack after this point will not be destructed correctly in the case of an error
		
#if !PLATFORM_EXCEPTIONS_DISABLED
		try
#endif
		{
			png_set_read_fn(png_ptr, this, FPngImageWrapper::user_read_compressed);

			png_read_info(png_ptr, info_ptr);

			Width = info_ptr->width;
			Height = info_ptr->height;
			ColorType = info_ptr->color_type;
			BitDepth = info_ptr->bit_depth;
			Channels = info_ptr->channels;
			if (info_ptr->valid & PNG_INFO_tRNS)
			{
				Format = ERGBFormat::RGBA;
			}
			else
			{
				Format = (ColorType & PNG_COLOR_MASK_COLOR || ColorType & PNG_COLOR_MASK_ALPHA) ? ERGBFormat::RGBA : ERGBFormat::Gray;
			}

			if (Format == ERGBFormat::RGBA && BitDepth <= 8)
			{
				Format = ERGBFormat::BGRA;
			}
			
			// Read all text blocks into our map
			TextBlocks.Reset();
			png_textp TextPointers = nullptr;
			int32 NumTextBlocks = 0;
			if (png_get_text(png_ptr, info_ptr, &TextPointers, &NumTextBlocks) != 0)
			{
				for (int32 Index = 0; Index < NumTextBlocks; Index++)
				{
					TextBlocks.Add(TextPointers[Index].key, TextPointers[Index].text);
				}
			}
		}
#if !PLATFORM_EXCEPTIONS_DISABLED
		catch (const FPNGImageCRCError& e)
		{
			/** 
			 *	libPNG has a known issue in version 1.5.2 causing
			 *	an unhandled exception upon a CRC error. This code 
			 *	catches our custom exception thrown in user_error_fn.
			 */
			UE_LOGF(LogImageWrapper, Error, "FPNGImageCRCError: %ls", *e.ErrorText);
			return false;
		}
#endif

		return true;
	}

	return false;
}

bool FPngImageWrapper::SupportsMetadata() const
{
	return true;
}

void FPngImageWrapper::AddMetadata(const FString& InKey, const FString& InValue)
{
	TextBlocks.Add(InKey, InValue);
}

void FPngImageWrapper::GetMetadata(TStringMap& OutMetadata) const
{
	OutMetadata = TextBlocks;
}

/* FPngImageWrapper static implementation
 *****************************************************************************/

void FPngImageWrapper::user_read_compressed(png_structp png_ptr, png_bytep data, png_size_t length)
{
	FPngImageWrapper* ctx = (FPngImageWrapper*)png_get_io_ptr(png_ptr);
	if (IntFitsIn<int64>(length) == false)
	{
		UE_LOGF(LogImageWrapper, Warning, "Bad PNG read length: %llu", length);
		ctx->SetError(TEXT("Invalid length in read_compressed")); // this doesn't seem to get logged on failure.
		return;
	}

	FGuardedInt64 GuardedEndOffset = FGuardedInt64(ctx->ReadOffset) + length;
	if (GuardedEndOffset.InvalidOrGreaterThan(ctx->CompressedData.Num()))
	{
		UE_LOG(LogImageWrapper, Warning, TEXT("Bad PNG read position: offset %" INT64_FMT " num %lld length: %llu"), ctx->ReadOffset, ctx->CompressedData.Num(), length);
		ctx->SetError(TEXT("Invalid read position for CompressedData.")); // this doesn't seem to get logged on failure.
		return;
	}

	FMemory::Memcpy(data, &ctx->CompressedData[ctx->ReadOffset], length);
	ctx->ReadOffset = GuardedEndOffset.Get(0);
}


void FPngImageWrapper::user_write_compressed(png_structp png_ptr, png_bytep data, png_size_t length)
{
	FPngImageWrapper* ctx = (FPngImageWrapper*) png_get_io_ptr(png_ptr);

	int64 Offset = ctx->CompressedData.AddUninitialized(length);
	FMemory::Memcpy(&ctx->CompressedData[Offset], data, length);
}


void FPngImageWrapper::user_flush_data(png_structp png_ptr)
{
}


void FPngImageWrapper::user_error_fn(png_structp png_ptr, png_const_charp error_msg)
{
	FPngImageWrapper* ctx = (FPngImageWrapper*)png_get_error_ptr(png_ptr);

	{
		FString ErrorMsg = ANSI_TO_TCHAR(error_msg);
		ctx->SetError(*ErrorMsg);

		UE_LOGF(LogImageWrapper, Error, "PNG Error(%ls): %ls", ctx->DebugImageName != nullptr ? ctx->DebugImageName : TEXT(""), *ErrorMsg);

	#if !PLATFORM_EXCEPTIONS_DISABLED
		/** 
		 *	libPNG has a known issue in version 1.5.2 causing 
		 *	an unhandled exception upon a CRC error. This code 
		 *	detects the error manually and throws our own 
		 *	exception to be handled. 
		 */
		if (ErrorMsg.Contains(TEXT("CRC error")))
		{
			throw FPNGImageCRCError(ErrorMsg);
		}
	#endif
	}

	// Ensure that FString is destructed prior to executing the longjmp

#if PLATFORM_ANDROID
	//Preserve old single thread code on some platform in relation to a type incompatibility at compile time.
	//The other platforms use libPNG jump buffer solution to allow concurrent compression\decompression on concurrent threads. The jump is trigered in libPNG after this function returns.
	longjmp(ctx->SetjmpBuffer, 1);
#endif

}


void FPngImageWrapper::user_warning_fn(png_structp png_ptr, png_const_charp warning_msg)
{
	FPngImageWrapper* ctx = (FPngImageWrapper*)png_get_error_ptr(png_ptr);
	const TCHAR* LocalDebugImageName = ctx != nullptr ? ctx->DebugImageName : nullptr;

	// libPNG 1.6+ has some very spammy warnings about incorrect color profile metadata that is unfortunately
	// extremely common in the wild, so manually detect and downgrade the verbosity of that
	if (FPlatformString::Strncmp(warning_msg, "iCCP:", 5) == 0)
	{
		UE_LOGF(LogImageWrapper, Verbose, "PNG Warning(%ls) %ls", LocalDebugImageName != nullptr ? LocalDebugImageName : TEXT(""), ANSI_TO_TCHAR(warning_msg));
	}
	else
	{
		UE_LOGF(LogImageWrapper, Warning, "PNG Warning(%ls) %ls", LocalDebugImageName != nullptr ? LocalDebugImageName : TEXT(""), ANSI_TO_TCHAR(warning_msg));
	}
}

void* FPngImageWrapper::user_malloc(png_structp /*png_ptr*/, png_size_t size)
{
	check(size > 0);
	return FMemory::Malloc(size);
}

void FPngImageWrapper::user_free(png_structp /*png_ptr*/, png_voidp struct_ptr)
{
	check(struct_ptr);
	FMemory::Free(struct_ptr);
}

// Renable warning "interaction between '_setjmp' and C++ object destruction is non-portable"
#ifdef _MSC_VER
	#pragma warning(pop)
#endif

#endif	//WITH_UNREALPNG
