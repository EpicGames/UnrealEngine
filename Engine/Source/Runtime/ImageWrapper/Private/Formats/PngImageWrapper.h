// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ImageWrapperBase.h"

#if WITH_UNREALPNG

THIRD_PARTY_INCLUDES_START
	#include "zlib.h"

	// make sure no other versions of libpng headers are picked up
#if WITH_LIBPNG_1_6
	#include "ThirdParty/libPNG/libPNG-1.6.44/png.h"
	#include "ThirdParty/libPNG/libPNG-1.6.44/pngstruct.h"
	#include "ThirdParty/libPNG/libPNG-1.6.44/pnginfo.h"
#else
	#include "ThirdParty/libPNG/libPNG-1.5.2/png.h"
	#include "ThirdParty/libPNG/libPNG-1.5.2/pnginfo.h"
#endif

	#include <setjmp.h>
THIRD_PARTY_INCLUDES_END


/**
 * PNG implementation of the helper class.
 *
 * The implementation of this class is almost entirely based on the sample from the libPNG documentation.
 * See http://www.libpng.org/pub/png/libpng-1.2.5-manual.html for details.
 *	
 * InitCompressed and InitRaw will set initial state and you will then be able to fill in Raw or
 * CompressedData by calling Uncompress or Compress respectively.
 */
class FPngImageWrapper
	: public FImageWrapperBase
{
public:

	/** Default Constructor. */
	FPngImageWrapper();

public:

	//~ FImageWrapper interface

	virtual void Compress(int32 Quality) override;
	virtual void Reset() override;
	virtual bool SetCompressed(const void* InCompressedData, int64 InCompressedSize) override;
	virtual void Uncompress(const ERGBFormat InFormat, int32 InBitDepth) override;
	
	virtual bool CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const override;
	virtual ERawImageFormat::Type GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const override;

	/**
	 * Does this image type support embedded metadata in its header?
	 *
	 * PNG is an example of an image type which supports adding user-defined metadata to its header.
	 */
	virtual bool SupportsMetadata() const override;

	/**
	 * Adds a key and value to this image's metadata.  Will be saved in the image's header and restored when the image is loaded.
	 *
	 * @param InKey Metadata consists of key value pairs.
	 * @param InValue Metadata consists of key value pairs.
	 */
	virtual void AddMetadata(const FString& InKey, const FString& InValue) override;

	/**
	 * Copies this image's metadata to the caller's map.
	 *
	 * @param OutMetadata A copy of this image's metadata.
	 */
	virtual void GetMetadata(TStringMap& OutMetadata) const override;

public:

	/** 
	 * Query whether this is a valid PNG type.
	 *
	 * @return true if data a PNG
	 */
	bool IsPNG() const;

	/** 
	 * Load the header information, returns true if successful.
	 *
	 * @return true if successful
	 */
	bool LoadPNGHeader();

	/** Helper function used to uncompress PNG data from a buffer */
	void UncompressPNGData(const ERGBFormat InFormat, const int32 InBitDepth);

protected:

	// Callbacks for the pnglibs
	static void  user_read_compressed(png_structp png_ptr, png_bytep data, png_size_t length);
	static void  user_write_compressed(png_structp png_ptr, png_bytep data, png_size_t length);
	static void  user_flush_data(png_structp png_ptr);
	static void  user_error_fn(png_structp png_ptr, png_const_charp error_msg);
	static void  user_warning_fn(png_structp png_ptr, png_const_charp warning_msg);
	static void* user_malloc(png_structp png_ptr, png_size_t size);
	static void  user_free(png_structp png_ptr, png_voidp struct_ptr);

private:

	/** The read offset into our array. */
	int64 ReadOffset;

	/** The color type as defined in the header. */
	int32 ColorType;

	/** The number of channels. */
	uint8 Channels;

#if PLATFORM_ANDROID
	//Other platforms rely on libPNG internal mechanism to achieve concurrent compression\decompression on multiple threads
	/** setjmp buffer for error recovery. */
	jmp_buf SetjmpBuffer;
#endif

	/** Metadata for/from our header. */
	TMap<FString, FString> TextBlocks;
};

#endif	//WITH_UNREALPNG
