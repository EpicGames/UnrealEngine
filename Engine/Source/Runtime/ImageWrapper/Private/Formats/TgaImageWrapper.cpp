// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formats/TgaImageWrapper.h"
#include "ImageWrapperPrivate.h"

#include "TgaImageSupport.h"
#include "ImageCoreUtils.h"



// CanSetRawFormat returns true if SetRaw will accept this format
bool FTgaImageWrapper::CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const
{
	return ( InFormat == ERGBFormat::BGRA ) && ( InBitDepth == 8 );
}

// returns InFormat if supported, else maps to something supported
ERawImageFormat::Type FTgaImageWrapper::GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const
{
	return ERawImageFormat::BGRA8;
}

void FTgaImageWrapper::Compress(int32 Quality)
{
	CompressedData.Reset();

	FImageView Image;
	if ( ! GetImageViewOfSetRawForCompress(Image) )
	{
		SetError(TEXT("No valid image to compress"));
		return;
	}
	
	if ( Width > UINT16_MAX || Height > UINT16_MAX )
	{
		SetError(TEXT("Image is too large to export as TGA"));
		return;
	}
			
	check( Image.Format == ERawImageFormat::BGRA8 );
	int64 ImageRowBytes = Width * 4;
	
	// write 32-bit if we have a non-all-255 alpha channel, else write 24 bit
	bool bHasAlpha = FImageCore::DetectAlphaChannel(Image);
	int64 OutBytesPerPixel = bHasAlpha ? 4 : 3;
	int64 OutSurfaceBytes = (int64) Width * Height * OutBytesPerPixel;

	int64 CompresedSize = sizeof(FTGAFileHeader) + OutSurfaceBytes;
	CompressedData.SetNum(CompresedSize);

	uint8 * CompressedPtr = CompressedData.GetData();
	
	// http://www.paulbourke.net/dataformats/tga/
	// https://en.wikipedia.org/wiki/Truevision_TGA
	FTGAFileHeader header = { };
	
	uint16 Width16  = IntCastChecked<uint16>( Width );
	uint16 Height16 = IntCastChecked<uint16>( Height );
	header.Width  = INTEL_ORDER16( Width16 );
	header.Height = INTEL_ORDER16( Height16 );
	header.ImageTypeCode = 2;
	header.BitsPerPixel = OutBytesPerPixel*8;

	// Image descriptor (1 byte): bits 3-0 give the alpha channel depth, bits 5-4 give pixel ordering
	header.ImageDescriptor = bHasAlpha ? 8 : 0;

	memcpy(CompressedPtr,&header,sizeof(header));
	CompressedPtr += sizeof(header);

	// write rows BGRA , bottom up :
	for(int32 y = Height-1; y>= 0;y--)
	{
		const uint8 * From = &RawData[y * ImageRowBytes]; 

		if ( bHasAlpha )
		{
			check( OutBytesPerPixel == 4 );
			check( RawData.Num() == OutSurfaceBytes );

			memcpy(CompressedPtr,From,ImageRowBytes);
			CompressedPtr += ImageRowBytes;
		}
		else
		{
			check( OutBytesPerPixel == 3 );
			
			for(int32 x=0;x<Width;x++)
			{
				// copy RGB, skip A
				CompressedPtr[0] = From[0];
				CompressedPtr[1] = From[1];
				CompressedPtr[2] = From[2];
				From += 4;
				CompressedPtr += 3;
			}
		}
	}

	// TGA file footer is optional :
	/*
	FTGAFileFooter Ftr;
	FMemory::Memzero( &Ftr, sizeof(Ftr) );
	FMemory::Memcpy( Ftr.Signature, "TRUEVISION-XFILE", 16 );
	Ftr.TrailingPeriod = '.';
	Ar.Serialize( &Ftr, sizeof(Ftr) );
	*/

	check( CompressedPtr == CompressedData.GetData() + CompressedData.Num() );
}

bool FTgaImageWrapper::SetCompressed(const void* InCompressedData, int64 InCompressedSize)
{
	bool bResult = FImageWrapperBase::SetCompressed(InCompressedData, InCompressedSize);

	if ( bResult && LoadTGAHeader() )
	{
		if ( ! FImageCoreUtils::IsImageImportPossible(Width,Height) )
		{
			SetError(TEXT("Image dimensions are not possible to import"));
			return false;
		}

		return true;
	}
	else
	{
		return false;
	}
}

void FTgaImageWrapper::Uncompress(const ERGBFormat InFormat, const int32 InBitDepth)
{
	const int32 BytesPerPixel = ( InFormat == ERGBFormat::Gray ? 1 : 4 );
	int64 TextureDataSize = (int64)Width * Height * BytesPerPixel;
	RawData.Empty(TextureDataSize);
	RawData.AddUninitialized(TextureDataSize);

	uint32* TextureData = reinterpret_cast< uint32* >( RawData.GetData() );

	FTGAFileHeader* TgaHeader = reinterpret_cast< FTGAFileHeader* >( CompressedData.GetData() );
	if ( !DecompressTGA_helper( TgaHeader, CompressedData.Num(), TextureData, TextureDataSize ) )
	{
		SetError(TEXT("Error while decompressing a TGA"));
		RawData.Reset();
	}
}

bool FTgaImageWrapper::IsTGAHeader(const void * CompressedData,int64 CompressedDataLength)
{
	const FTGAFileHeader* TgaHeader = (FTGAFileHeader*)CompressedData;

	if ( CompressedDataLength >= sizeof( FTGAFileHeader ) &&
		( (TgaHeader->ColorMapType == 0 && TgaHeader->ImageTypeCode == 2) ||
		  ( TgaHeader->ColorMapType == 0 && TgaHeader->ImageTypeCode == 3 ) || // ImageTypeCode 3 is greyscale
		  ( TgaHeader->ColorMapType == 0 && TgaHeader->ImageTypeCode == 10 ) ||
		  ( TgaHeader->ColorMapType == 1 && TgaHeader->ImageTypeCode == 1 && TgaHeader->BitsPerPixel == 8 ) ) )
	{
		return true;
	}
	else
	{
		return false;
	}
}

bool FTgaImageWrapper::LoadTGAHeader()
{
	check( CompressedData.Num() );

	if( ! IsTGAHeader(CompressedData.GetData(),CompressedData.Num()) )
	{
		return false;
	}

	const FTGAFileHeader* TgaHeader = (FTGAFileHeader*)CompressedData.GetData();

	Width = TgaHeader->Width;
	Height = TgaHeader->Height;
	BitDepth = 8;
	
	// must exactly match the logic in DecompressTGA_helper
	if(TgaHeader->ColorMapType == 1 && TgaHeader->ImageTypeCode == 1 && TgaHeader->BitsPerPixel == 8)
	{
		Format = ERGBFormat::Gray;
	}
	else if(TgaHeader->ColorMapType == 0 && TgaHeader->ImageTypeCode == 3 && TgaHeader->BitsPerPixel == 8)
	{
		Format = ERGBFormat::Gray;
	}
	else
	{
		Format = ERGBFormat::BGRA;
	}

	ColorMapType = TgaHeader->ColorMapType;
	ImageTypeCode = TgaHeader->ImageTypeCode;
	
	// TgaHeader->BitsPerPixel is bits per *color*

	return true;
}
