// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ImageWrapperBase.h"


/**
 * TGA implementation of the helper class
 */
class FTgaImageWrapper : public FImageWrapperBase
{

public:
	/** 
	 * Load the header information, returns true if successful.
	 */
	bool LoadTGAHeader();
	
	static bool IsTGAHeader(const void * CompressedData,int64 CompressedDataLength);

public:

	//~ FImageWrapper interface

	virtual void Compress(int32 Quality) override;
	virtual bool SetCompressed(const void* InCompressedData, int64 InCompressedSize) override;
	virtual void Uncompress(const ERGBFormat InFormat, int32 InBitDepth) override;
	
	virtual bool CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const override;
	virtual ERawImageFormat::Type GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const override;

private:
	/** The color type as defined in the header. */
	int32 ColorMapType;
	uint8 ImageTypeCode;
};
