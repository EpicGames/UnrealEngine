// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formats/TiffImageWrapper.h"

#if WITH_LIBTIFF // set from libtiff.build.cs , available on all tool platforms

#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Queue.h"
#include "CoreMinimal.h"
#include "ImageWrapperPrivate.h"
#include "Math/Float16.h"
#include "Math/GuardedInt.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/CoreStats.h"
#include "Templates/IsSigned.h"
#include "Templates/UnrealTypeTraits.h"
#include "ImageCoreUtils.h"

#include <type_traits>

THIRD_PARTY_INCLUDES_START
#include "tiff.h"
#include "tiffio.h"
THIRD_PARTY_INCLUDES_END

namespace UE::ImageWrapper::Private
{
	namespace TiffImageWrapper
	{
		template<class DataTypeDest, class DataTypeSrc>
		DataTypeDest ConvertToWriteFormat(DataTypeSrc ReadValue)
		{
			if constexpr (std::is_same_v<DataTypeDest, DataTypeSrc>)
			{
				return ReadValue;
			}
			else
			{
				// we almost always load the tiff into the same format it was stored in
				// so this conversion is rarely used
				// one case where it is used is for int32 to int16

				// get source value in [0,1]
				double SrcValue;
				if constexpr (std::is_same_v<DataTypeSrc, FFloat16> || TIsFloatingPoint<DataTypeSrc>::Value)
				{
					SrcValue = FMath::Clamp((double)ReadValue, 0.0, 1.0);
				}
				else
				{
					SrcValue = ReadValue * (1.0 / (double)TNumericLimits<DataTypeSrc>::Max() );
				}
				
				// quantize to dest :
				if constexpr (std::is_same_v<DataTypeDest, FFloat16> || TIsFloatingPoint<DataTypeDest>::Value)
				{
					return DataTypeDest(SrcValue);
				}
				else
				{
					return (DataTypeDest)( SrcValue * (double)TNumericLimits<DataTypeDest>::Max() + 0.50 );
				}
			}
		}

		/**
		 * Does the actual reading and writing
		 * Manage the specifics of each channel type read, conversion to their destination type and writing to the output buffer
		 */
		template<class DataTypeDest, class DataTypeSrc>
		struct TDefaultAdapter
		{
			TDefaultAdapter(TIFF* Tiff)
			{
			}

			void ReadAndWrite(TArrayView64<const DataTypeSrc>& ReadArray, int64 ReadIndex, TArrayView64<DataTypeDest>& WriteArray, int64 WriteIndex)
			{
				WriteArray[WriteIndex] = ConvertToWriteFormat<DataTypeDest, DataTypeSrc>(ReadArray[ReadIndex]);
			}

			static const uint8 ChannelPerReadIndex = 1;
		};

		// ===== Small Read adapter =====

		// It should be used as parent to another adapter to provide the ReadAndWrite function
		template<uint8 NumBits>
		struct TReadBitsBaseAdapter
		{
			TReadBitsBaseAdapter(TIFF* Tiff)
			{
				LineSizeScr = TIFFScanlineSize64(Tiff);
				TIFFGetField(Tiff, TIFFTAG_IMAGEWIDTH, &Width);
			}

			uint8 Read(TArrayView64<const uint8>& ReadArray, int64 ReadIndex)
			{
				constexpr uint8 BaseMask = uint8(0xff) >> ((ChannelPerReadIndex - 1) * NumBits);
				
				const uint32 Row = ReadIndex / Width;
				const uint32 Column = ReadIndex % Width;

				const uint32 PositionOfColInBuffer = Column / ChannelPerReadIndex;

				/**
				 * The data is stored in reverse order in the byte. From highest to lowest
				 * Example for eight bits from most significant to less (0, 1, 2, 3, 4, 5, 6, 7)
				 */
				const uint32 PositionInByte = Column % ChannelPerReadIndex;

				const uint8 NumShift = ((ChannelPerReadIndex - 1) - PositionInByte) * NumBits;
				const uint8 Mask = BaseMask << NumShift;

				ReadIndex = LineSizeScr * Row + PositionOfColInBuffer;

				const uint8 RawValues = ReadArray[ReadIndex];
				const uint8 RawValue = RawValues & Mask;

				return RawValue >> NumShift;
			}

			uint64 LineSizeScr = 0;
			uint32 Width = 0;

			static const uint8 ChannelPerReadIndex = 8 / NumBits;
		};

		template struct TReadBitsBaseAdapter<1>;
		template struct TReadBitsBaseAdapter<2>;
		template struct TReadBitsBaseAdapter<4>;

		// ===== Tiff Small bit to uint8 adapter =====
		template<uint8 NumBits>
		struct TWriteBitsToUint8Adaptor : public TReadBitsBaseAdapter<NumBits>
		{
			using Parent = TReadBitsBaseAdapter<NumBits>;

			TWriteBitsToUint8Adaptor(TIFF* Tiff)
				: Parent(Tiff)
			{
			}

			void ReadAndWrite(TArrayView64<const uint8>& ReadArray, int64 ReadIndex, TArrayView64<uint8>& WriteArray, int64 WriteIndex)
			{
				constexpr uint8 MaxValue = uint8(0xff) >> ((Parent::ChannelPerReadIndex - 1) * NumBits);

				const uint8 ReadValue = Parent::Read(ReadArray, ReadIndex);
				WriteArray[WriteIndex] = TNumericLimits<uint8>::Max() * (double(ReadValue) / MaxValue);
			}
		};

		// ===== Tiff Palette adapters =====

		template<class DataTypeSrc>
		struct TPaletteBaseAdapter
		{
			TPaletteBaseAdapter(TIFF* Tiff)
			{
				uint16* RedsPtr = nullptr;
				uint16* GreensPtr = nullptr;
				uint16* BluesPtr = nullptr;
				TIFFGetField(Tiff, TIFFTAG_COLORMAP, &RedsPtr, &GreensPtr, &BluesPtr); 

				check (RedsPtr && GreensPtr && BluesPtr);

				uint16 BitsPerSample;
				TIFFGetField(Tiff, TIFFTAG_BITSPERSAMPLE, &BitsPerSample);
				int64 NumValues = int64(1) << BitsPerSample;

				Reds = TArrayView64<uint16>(RedsPtr, NumValues);
				Greens = TArrayView64<uint16>(GreensPtr, NumValues);
				Blues = TArrayView64<uint16>(BluesPtr, NumValues);
			}

			void Write(TArrayView64<uint16>& WriteArray, int64 WriteIndex, DataTypeSrc ReadValue)
			{
				WriteArray[WriteIndex] = Reds[ReadValue];
				WriteArray[WriteIndex + 1] = Greens[ReadValue];
				WriteArray[WriteIndex + 2] = Blues[ReadValue];
			}

			TArrayView64<uint16> Reds;
			TArrayView64<uint16> Greens;
			TArrayView64<uint16> Blues;
		};

		template struct TPaletteBaseAdapter<uint8>;
		template struct TPaletteBaseAdapter<uint16>;
		template struct TPaletteBaseAdapter<uint32>;

		template<class DataTypeSrc>
		struct TPaletteAdapter : public TPaletteBaseAdapter<DataTypeSrc>
		{
			using Parent = TPaletteBaseAdapter<DataTypeSrc>;

			TPaletteAdapter(TIFF* Tiff)
				: Parent(Tiff)
			{
			}

			void ReadAndWrite(TArrayView64<const DataTypeSrc>& ReadArray, int64 ReadIndex, TArrayView64<uint16>& WriteArray, int64 WriteIndex)
			{
				Parent::Write(WriteArray, WriteIndex, ReadArray[ReadIndex]);
			}

			static const uint8 ChannelPerReadIndex = 1;
		};

		
		template<uint8 NumBits>
		struct TPaletteBitsAdapter : public TPaletteBaseAdapter<uint8>, public TReadBitsBaseAdapter<NumBits>
		{
			using Reader = TReadBitsBaseAdapter<NumBits>;

			TPaletteBitsAdapter(TIFF* Tiff)
				: TPaletteBaseAdapter<uint8>(Tiff)
				, Reader(Tiff)
			{
			}

			void ReadAndWrite(TArrayView64<const uint8>& ReadArray, int64 ReadIndex, TArrayView64<uint16>& WriteArray, int64 WriteIndex)
			{
				Write(WriteArray, WriteIndex, Reader::Read(ReadArray, ReadIndex));
			}
		};

		// ===== Convert Grayscale two channel to RGBA =====
		template<class ParentApdater, class DataTypeDest, class DataTypeSrc>
		struct TTwoChannelToFourAdapter : public ParentApdater
		{
			TTwoChannelToFourAdapter(TIFF* Tiff)
				: ParentApdater(Tiff)
			{
			}

			void ReadAndWrite(TArrayView64<const DataTypeSrc>& ReadArray, int64 ReadIndex, TArrayView64<DataTypeDest>& WriteArray, int64 WriteIndex)
			{
				bool IsAlpha = WriteIndex % 2 == 1;
			
				if (IsAlpha)
				{
					ParentApdater::ReadAndWrite(ReadArray, ReadIndex, WriteArray, WriteIndex + 2);
				}
				else
				{
					ParentApdater::ReadAndWrite(ReadArray, ReadIndex, WriteArray, WriteIndex);
					DataTypeDest ValueWritten = WriteArray[WriteIndex];
					WriteArray[WriteIndex + 1] = ValueWritten;
					WriteArray[WriteIndex + 2] = ValueWritten;
				}
			}
		};


		class FProcessDecodedDataTask
		{
		public:
			public:

			/**
			 * Creates and initializes a new instance.
			 *
			 * @param InFunction The function to execute asynchronously.
			 */
			FProcessDecodedDataTask(TUniqueFunction<void()>&& InFunction)
				: Function(MoveTemp(InFunction))
				, DesiredThread(IsInGameThread() ? ENamedThreads::AnyHiPriThreadHiPriTask : ENamedThreads::AnyBackgroundThreadNormalTask)
			{}

		public:

			/**
			 * Performs the actual task.
			 *
			 * @param CurrentThread The thread that this task is executing on.
			 * @param MyCompletionGraphEvent The completion event.
			 */
			void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
			{
				Function();
			}

			/**
			 * Returns the name of the thread that this task should run on.
			 *
			 * @return Always run on any thread.
			 */
			ENamedThreads::Type GetDesiredThread()
			{
				return DesiredThread;
			}

			/**
			 * Gets the task's stats tracking identifier.
			 *
			 * @return Stats identifier.
			 */
			TStatId GetStatId() const
			{
				return GET_STATID(STAT_TaskGraph_OtherTasks);
			}

			/**
			 * Gets the mode for tracking subsequent tasks.
			 *
			 * @return Always track subsequent tasks.
			 */
			static ESubsequentsMode::Type GetSubsequentsMode()
			{
				return ESubsequentsMode::TrackSubsequents;
			}

		private:
			TUniqueFunction<void()> Function;
			ENamedThreads::Type DesiredThread;
		};

	}

	class FScopedTiffAllocation
	{
	public:
		FScopedTiffAllocation(int64 BytesSize)
		{
			Allocation = _TIFFmalloc(BytesSize);
		}

		~FScopedTiffAllocation()
		{
			_TIFFfree(Allocation);
		}

		void* Allocation = nullptr;
	};

	struct FTIFFReadMemoryFile
	{
		static tmsize_t Read(thandle_t Handle, void* Buffer, tmsize_t Size)
		{
			FTiffImageWrapper* TiffImageWrapper = reinterpret_cast<FTiffImageWrapper*>(Handle);
			if (TiffImageWrapper->CurrentPosition < 0 || TiffImageWrapper->CurrentPosition >= TiffImageWrapper->CompressedData.Num())
			{
				return 0;
			}

			if (IntFitsIn<int64>(Size) == false)
			{
				Size = TNumericLimits<int64>::Max();
			}

			// We now know this is [0, Num()].
			int64 RemainingBytes = TiffImageWrapper->CompressedData.Num() - TiffImageWrapper->CurrentPosition;
			int64 NumBytesRead = FMath::Min<int64>(RemainingBytes, Size);

			FMemory::Memcpy(Buffer, TiffImageWrapper->CompressedData.GetData() + TiffImageWrapper->CurrentPosition, NumBytesRead);
			TiffImageWrapper->CurrentPosition += NumBytesRead;

			return NumBytesRead;
		}

		static tmsize_t Write(thandle_t Handle, void* Buffer, tmsize_t Size)
		{
			// We don't support writing to a in memory file
			return -1;
		}

		static toff_t Seek(thandle_t Handle, toff_t Offset, int Whence)
		{
			FTiffImageWrapper* TiffImageWrapper = reinterpret_cast<FTiffImageWrapper*>(Handle);

			const int Set = 0;
			const int OffsetFromCurrent = 1;
			const int FromEnd = 2;

			FGuardedInt64 TargetPosition;
			switch (Whence)
			{
				case Set:
					{
						TargetPosition = FGuardedInt64(Offset);
						break;
					}
				case OffsetFromCurrent:
					{
						TargetPosition = FGuardedInt64(TiffImageWrapper->CurrentPosition) + Offset;
						break;
					}
				case FromEnd:
					{
						TargetPosition = FGuardedInt64(TiffImageWrapper->CompressedData.Num()) + Offset;
						break;
					}
				default:
					return -1;
			}

			TiffImageWrapper->CurrentPosition = FMath::Max(0, TargetPosition.Get(TiffImageWrapper->CurrentPosition));
			return TiffImageWrapper->CurrentPosition;
		}

		static toff_t GetSize(thandle_t Handle)
		{
			FTiffImageWrapper* TiffImageWrapper = reinterpret_cast<FTiffImageWrapper*>(Handle);
			return TiffImageWrapper->CompressedData.Num();
		}

		static int Close(thandle_t Handle)
		{
			return 0;
		}

		static int MapFile(thandle_t Handle, void** Base, toff_t* Size)
		{
			return 0;
		}

		static void UnmapFile(thandle_t Handle, void* base, toff_t Size)
		{
		}
	};

	FTiffImageWrapper::~FTiffImageWrapper()
	{
		ReleaseTiffImage();
	}

	void FTiffImageWrapper::Compress(int32 Quality)
	{
		// should not get here because CanSetRawFormat is false
		checkf(false, TEXT("TIFF compression not supported"));
	}
	
	// CanSetRawFormat returns true if SetRaw will accept this format
	bool FTiffImageWrapper::CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const
	{
		return false;
	}

	// returns InFormat if supported, else maps to something supported
	ERawImageFormat::Type FTiffImageWrapper::GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const
	{
		return ERawImageFormat::BGRA8;
	}

	void FTiffImageWrapper::Uncompress(const ERGBFormat InFormat, int32 InBitDepth)
	{
		ON_SCOPE_EXIT{
			ReleaseTiffImage();
		};

		if (Tiff == nullptr)
		{
			SetError(TEXT("Tiff invalid."));
			return;
		}

		// We just want mip 0
		TMap<FString, FString> OutTagMetaData;
		const bool bUncompressResult = Uncompress_Internal(InFormat, InBitDepth, OutTagMetaData);
		if (bUncompressResult && SubImageBuffer.Num())
		{
			// Append the current sub-image to the base byte buffer that reads all the 
			RawData.Append(MoveTemp(SubImageBuffer));
		}
		else
		{
			Width = 0;
			Height = 0;
		}

		CurrSubImageWidth = 0;
		CurrSubImageHeight = 0;
	}

	void FTiffImageWrapper::Uncompress(const ERGBFormat InFormat, int32 InBitDepth, FDecompressedImageOutput& OutDecompressedImage)
	{
		ON_SCOPE_EXIT{
			ReleaseTiffImage();
		};

		if (Tiff == nullptr)
		{
			SetError(TEXT("Tiff invalid."));
			return;
		}

		do
		{
			TMap<FString, FString> OutTagMetaData;
			const bool bUncompressResult = Uncompress_Internal(InFormat, InBitDepth, OutTagMetaData);
			if (OutDecompressedImage.ApplicationName.IsEmpty())
			{
				OutDecompressedImage.ApplicationName = OutTagMetaData.FindRef(TEXT("Software"),TEXT(""));
			}

			if (bUncompressResult && SubImageBuffer.Num())
			{
				static bool bExactMatch = false;

				const int64 SubImageOffset = RawData.Num();
				// Append the current sub-image to the base byte buffer that reads all the 
				OutDecompressedImage.MipMapImage.AddMipImage(MoveTemp(SubImageBuffer), CurrSubImageWidth, CurrSubImageHeight);
			}
			else
			{
				Width = 0;
				Height = 0;
				break;
			}

		} while (TIFFReadDirectory(Tiff)); // This will go to the next directory if available. Next Directory will contain the next sub-image.

		CurrSubImageWidth = 0;
		CurrSubImageHeight = 0;
	}

	bool FTiffImageWrapper::Uncompress_Internal(const ERGBFormat InFormat, int32 InBitDepth, TMap<FString, FString>& OutTagMetaData)
	{
		// SubImageBuffer gets emptied in the respective calls to UnpackIntoRawBuffer
		TIFFGetField(Tiff, TIFFTAG_IMAGEWIDTH, &CurrSubImageWidth);
		TIFFGetField(Tiff, TIFFTAG_IMAGELENGTH, &CurrSubImageHeight);

		const int32 TagCount = TIFFGetTagListCount(Tiff);
		OutTagMetaData.Reserve(TagCount);
		for (int32 TagCountIndex = 0; TagCountIndex < TagCount; ++TagCountIndex)
		{
			uint32 TagListEntry = TIFFGetTagListEntry(Tiff, TagCountIndex);
			const TIFFField* TiffField = TIFFFieldWithTag(Tiff, TagListEntry);
			if (!TiffField)
			{
				continue;
			}

			TIFFDataType TiffDataType = TIFFFieldDataType(TiffField);

			if (TiffDataType != TIFF_ASCII)
			{
				continue;
			}

			const int32 PassCount = TIFFFieldPassCount(TiffField);
			const int32 ReadCount = TIFFFieldReadCount(TiffField);

			char* FieldCString = nullptr;
			bool bCanProcessField = false;
			
			if (PassCount > 0 && ReadCount == TIFF_VARIABLE)
			{
				uint16 SmallValueCount = 0;
				bCanProcessField = TIFFGetField(Tiff, TagListEntry, &SmallValueCount, &FieldCString) == 1;
			}
			else if (PassCount > 0 && ReadCount == TIFF_VARIABLE2)
			{
				uint32 FieldValueCount = 0;
				bCanProcessField = TIFFGetField(Tiff, TagListEntry, &FieldValueCount, &FieldCString) == 1;
			}
			else if (ReadCount > 0)
			{
				bCanProcessField = TIFFGetField(Tiff, TagListEntry, &FieldCString) == 1;
			}
			else
			{
				bCanProcessField = TIFFGetField(Tiff, TagListEntry, &FieldCString) == 1;
			}
			
			if (bCanProcessField && FieldCString && *FieldCString)
			{
				// Skip anonymous fields for now.
				if (const char* FieldName = TIFFFieldName(TiffField))
				{
					OutTagMetaData.Emplace(FieldName, (const char*)FieldCString);
				}
			}
		}

		if (InFormat == Format && InBitDepth == BitDepth)
		{
			// Read using RGBA
			if (Format == ERGBFormat::BGRA && BitDepth == 8)
			{
				FGuardedInt64 GuardedBufferSize = FGuardedInt64(CurrSubImageWidth) * CurrSubImageHeight * sizeof(uint32);
				if (GuardedBufferSize.IsValid() == false)
				{
					SetError(TEXT("Tiff image is massive - likely corrupted file."));
					return false;
				}

				const int64 BufferSize = GuardedBufferSize.Get(0);
				const int Flags = 0;

				SubImageBuffer.Empty(BufferSize);
				SubImageBuffer.AddUninitialized(BufferSize);

				if (TIFFReadRGBAImageOriented(Tiff, CurrSubImageWidth, CurrSubImageHeight, static_cast<uint32*>(static_cast<void*>(SubImageBuffer.GetData())), ORIENTATION_LEFTTOP, 0) != 0)
				{
					// @@ use ImageCore TransposeImageRGBABGRA
					EParallelForFlags ParallelForFlags = IsInGameThread() ? EParallelForFlags::None : EParallelForFlags::BackgroundPriority;
					ParallelFor(CurrSubImageHeight, [this](int32 HeightIndex)
						{
							const int64 HeightOffset = HeightIndex * CurrSubImageWidth;
							for (int32 WidthIndex = 0; WidthIndex < CurrSubImageWidth; WidthIndex++)
							{
								// Swap from RGBA to BGRA
								const int64 CurrentPixelIndex = (WidthIndex + HeightOffset) * sizeof(uint32);
								uint8 Red = SubImageBuffer[CurrentPixelIndex];
								SubImageBuffer[CurrentPixelIndex] = SubImageBuffer[CurrentPixelIndex + 2];
								SubImageBuffer[CurrentPixelIndex + 2] = Red;
							}
						}
					, ParallelForFlags);
				}
				else
				{
					UnpackIntoRawBuffer<uint8>(4);
				}
			}
			else if (Format == ERGBFormat::RGBA && BitDepth == 16)
			{
				UnpackIntoRawBuffer<uint16>(4);
			}
			else if (Format == ERGBFormat::RGBAF && BitDepth == 16)
			{
				UnpackIntoRawBuffer<FFloat16>(4);
			}
			else if (Format == ERGBFormat::RGBAF && BitDepth == 32)
			{
				UnpackIntoRawBuffer<float>(4);
			}
			else if (Format == ERGBFormat::Gray && BitDepth == 8)
			{
				UnpackIntoRawBuffer<uint8>(1);
			}
			else if (Format == ERGBFormat::Gray && BitDepth == 16)
			{
				UnpackIntoRawBuffer<uint16>(1);
			}
			else if (Format == ERGBFormat::GrayF && BitDepth == 16)
			{
				UnpackIntoRawBuffer<FFloat16>(1);
			}
			else if (Format == ERGBFormat::GrayF && BitDepth == 32)
			{
				UnpackIntoRawBuffer<float>(1);
			}
			else
			{
				SetError(TEXT("Unsupported requested format for the input image. Can't uncompress the tiff image."));
				return false;
			}
		}
		else
		{
			SetError(TEXT("Unsupported requested format for the input image. Can't uncompress the tiff image."));
			return false;
		}

		return true;
	}

	bool FTiffImageWrapper::SetCompressed(const void* InCompressedData, int64 InCompressedSize)
	{
		if ( !  FImageWrapperBase::SetCompressed( InCompressedData, InCompressedSize ) )
		{
			return false;
		}

		{
			Tiff = TIFFClientOpen(""
				, "r"
				, this
				, FTIFFReadMemoryFile::Read
				, FTIFFReadMemoryFile::Write
				, FTIFFReadMemoryFile::Seek
				, FTIFFReadMemoryFile::Close
				, FTIFFReadMemoryFile::GetSize
				, FTIFFReadMemoryFile::MapFile
				, FTIFFReadMemoryFile::UnmapFile);

			if (Tiff)
			{
				TIFFGetField(Tiff, TIFFTAG_IMAGEWIDTH, &Width);
				TIFFGetField(Tiff, TIFFTAG_IMAGELENGTH, &Height);
				TIFFGetField(Tiff, TIFFTAG_PHOTOMETRIC, &Photometric);
				TIFFGetField(Tiff, TIFFTAG_COMPRESSION, &Compression);
				TIFFGetField(Tiff, TIFFTAG_SAMPLESPERPIXEL, &SamplesPerPixel);
				TIFFGetField(Tiff, TIFFTAG_BITSPERSAMPLE, &BitsPerSample);
				TIFFGetField(Tiff, TIFFTAG_SAMPLEFORMAT, &SampleFormat);

				Format = ERGBFormat::Invalid;
				
				if (Width <= 0 ||
					Height <= 0)
				{
					SetError(TEXT("Invalid resolution in tiff: zero or negative."));
					return false;
				}

				if ( SampleFormat != SAMPLEFORMAT_UINT
					&& SampleFormat != SAMPLEFORMAT_IEEEFP
					&& SampleFormat != 0 /* assume it's uint */)
				{
					SetError(TEXT("The sample format of the tiff is unsuported. (We support UInt and IEEEFP)"));
					return false;
				}

				switch (Photometric)
				{
				case PHOTOMETRIC_RGB:
					if ( SamplesPerPixel == 3 || SamplesPerPixel == 4)
					{
						if (BitsPerSample == 16 || BitsPerSample == 32 || BitsPerSample == 64)
						{
							if (SampleFormat == SAMPLEFORMAT_IEEEFP )
							{
								Format = ERGBFormat::RGBAF;
								
								BitDepth = ( BitsPerSample >= 32 ) ? 32 : 16;
							}
							else
							{
								Format = ERGBFormat::RGBA;

								// we don't have a 32-bit integer texture format, so always load as 16 bit								
								BitDepth = 16;
							}
						}
						else if ( BitsPerSample <= 8 )
						{
							// Will be converted to 8 bits per channel
							Format = ERGBFormat::BGRA;
							BitDepth = 8;
						}
					}
					break;

				case PHOTOMETRIC_YCBCR:
				case PHOTOMETRIC_CIELAB:
				case PHOTOMETRIC_ICCLAB:
				case PHOTOMETRIC_ITULAB:
					Format = ERGBFormat::BGRA;
					BitDepth = 8;
					break;

				case PHOTOMETRIC_MINISBLACK:
				case PHOTOMETRIC_MINISWHITE:
					if (SamplesPerPixel == 1 || SamplesPerPixel == 2) // 2 ?
					{
						if (BitsPerSample == 1 || BitsPerSample == 2 || BitsPerSample == 4 || BitsPerSample == 8)
						{
							if (SamplesPerPixel == 1)
							{
								Format = ERGBFormat::Gray;
							}
							else
							{
								Format = ERGBFormat::BGRA;
							}
							BitDepth = 8;
						}
						else if (BitsPerSample == 16 || BitsPerSample == 32 || BitsPerSample == 64)
						{
							if (SamplesPerPixel == 1)
							{
								if (SampleFormat == SAMPLEFORMAT_IEEEFP)
								{
									Format = ERGBFormat::GrayF;
								}
								else
								{
									Format = ERGBFormat::Gray;
								}
							}
							else
							{
								if (SampleFormat == SAMPLEFORMAT_IEEEFP)
								{
									Format = ERGBFormat::RGBAF;
								}
								else
								{
									Format = ERGBFormat::RGBA;
								}
							}

							if (SampleFormat == SAMPLEFORMAT_IEEEFP && BitsPerSample >= 32 )
							{
								BitDepth = 32;
							}
							else
							{
								// we don't have an int32 texture format, so they convert to 16 bit
								BitDepth = 16;
							}
						}
					}
					break;

				case PHOTOMETRIC_PALETTE:
					if (SamplesPerPixel == 1)
					{ 
						if (BitsPerSample == 1
							|| BitsPerSample == 2
							|| BitsPerSample == 4
							|| BitsPerSample == 8)
						{
							Format = ERGBFormat::BGRA;
							BitDepth = 8;
						}
						else if ( BitsPerSample == 16
							|| BitsPerSample == 32
							|| BitsPerSample == 64)
						{
							Format = ERGBFormat::RGBA;
							BitDepth = 16;
						}
					}
					break;

				default:
					break;
				}

				if (Format == ERGBFormat::Invalid)
				{
					SetError(TEXT("Unsupported Tiff content."));
					return false;
				}
				
				// note: the "Tiff" object is retained until the Uncompress call
			}
			else
			{
				return false;
			}
		}

		if ( ! FImageCoreUtils::IsImageImportPossible(Width,Height) )
		{
			SetError(TEXT("Image dimensions are not possible to import"));
			return false;
		}

		return true;

	}

	

	void FTiffImageWrapper::ReleaseTiffImage()
	{
		if (Tiff)
		{
			TIFFClose(Tiff);
			Tiff = nullptr;
		}
	}



	template<class DataTypeDest, class DataTypeSrc, class Adapter>
	void FTiffImageWrapper::CallUnpackIntoRawBufferImpl(uint8 NumOfChannelDest, const bool bIsTiled)
	{
		const bool bShouldAddAlpha = NumOfChannelDest == 4 && SamplesPerPixel == 3;
		if (NumOfChannelDest == SamplesPerPixel || bShouldAddAlpha)
		{
			if (bIsTiled)
			{
				UnpackIntoRawBufferImpl<DataTypeDest, DataTypeSrc, true, Adapter>
				(NumOfChannelDest, bShouldAddAlpha);
			}
			else
			{
				UnpackIntoRawBufferImpl<DataTypeDest, DataTypeSrc, false, Adapter>
				(NumOfChannelDest, bShouldAddAlpha);
			}
		}
		else if (SamplesPerPixel == 2 && NumOfChannelDest == 4)
		{
			if (bIsTiled)
			{
				UnpackIntoRawBufferImpl<DataTypeDest, DataTypeSrc, true, TiffImageWrapper::TTwoChannelToFourAdapter<Adapter, DataTypeDest, DataTypeSrc>>
				(NumOfChannelDest, false);
			}
			else
			{
				UnpackIntoRawBufferImpl<DataTypeDest, DataTypeSrc, false, TiffImageWrapper::TTwoChannelToFourAdapter<Adapter, DataTypeDest, DataTypeSrc>>
				(NumOfChannelDest, false);
			}
		}
		else
		{
			SetError(TEXT("Tiff, Unsupported conversion of sample data format."));
		}
	}

	template<class DataTypeDest, class DataTypeSrc>
	void FTiffImageWrapper::DefaultCallUnpackIntoRawBufferImpl(uint8 NumOfChannelDest, const bool bIsTiled)
	{
		CallUnpackIntoRawBufferImpl<DataTypeDest, DataTypeSrc, TiffImageWrapper::TDefaultAdapter<DataTypeDest, DataTypeSrc>>(NumOfChannelDest, bIsTiled);
	}

	template<class DataTypeDest, class DataTypeSrc, class Adapter>
	void FTiffImageWrapper::PaletteCallUnpackIntoRawBufferImpl(uint8 NumOfChannelDest, const bool bIsTiled)
	{
		if (bIsTiled)
		{
			UnpackIntoRawBufferImpl<DataTypeDest, DataTypeSrc, true, Adapter>
				(NumOfChannelDest, true);
		}
		else
		{
			UnpackIntoRawBufferImpl<DataTypeDest, DataTypeSrc, false, Adapter>
				(NumOfChannelDest, true);
		}
	}


	template<class DataTypeDest>
	void FTiffImageWrapper::UnpackIntoRawBuffer(const uint8 NumOfChannelDest)
	{
		const int64 PixelSize = sizeof(DataTypeDest) * NumOfChannelDest;
		const int64 RowSize = PixelSize * int64(CurrSubImageWidth);
		const int64 BufferSize = RowSize * int64(CurrSubImageHeight);
		const int Flags = 0;
	
		SubImageBuffer.Empty(BufferSize);
		SubImageBuffer.AddUninitialized(BufferSize);

		const bool bIsTiled = TIFFIsTiled(Tiff) != 0;

		if constexpr (std::is_same_v<DataTypeDest, uint8>)
		{
			if (BitsPerSample == 1)
			{
				CallUnpackIntoRawBufferImpl<DataTypeDest, uint8, TiffImageWrapper::TWriteBitsToUint8Adaptor<1>>(NumOfChannelDest, bIsTiled);
			}
			else if (BitsPerSample == 2)
			{
				CallUnpackIntoRawBufferImpl<DataTypeDest, uint8, TiffImageWrapper::TWriteBitsToUint8Adaptor<2>>(NumOfChannelDest, bIsTiled);
			}
			else if (BitsPerSample == 4)
			{
				CallUnpackIntoRawBufferImpl<DataTypeDest, uint8, TiffImageWrapper::TWriteBitsToUint8Adaptor<4>>(NumOfChannelDest, bIsTiled);
			}
			else if (BitsPerSample == 8)
			{
				DefaultCallUnpackIntoRawBufferImpl<DataTypeDest, uint8>(NumOfChannelDest, bIsTiled);
			}
			else
			{
				SetError(TEXT("Tiff, Unsupported bits per sample from a uint sample format."));
			}
		}
		else
		{
			if (SampleFormat == SAMPLEFORMAT_UINT || SampleFormat == 0)
			{
				if (Photometric == PHOTOMETRIC_PALETTE)
				{
					if constexpr (std::is_same_v<DataTypeDest, uint16>)
					{
						switch (BitsPerSample)
						{
						case 1:
							PaletteCallUnpackIntoRawBufferImpl<DataTypeDest, uint8, TiffImageWrapper::TPaletteBitsAdapter<1>>(NumOfChannelDest, bIsTiled);
							break;
						case 2:
							PaletteCallUnpackIntoRawBufferImpl<DataTypeDest, uint8, TiffImageWrapper::TPaletteBitsAdapter<2>>(NumOfChannelDest, bIsTiled);
							break;
						case 4:
							PaletteCallUnpackIntoRawBufferImpl<DataTypeDest, uint8, TiffImageWrapper::TPaletteBitsAdapter<4>>(NumOfChannelDest, bIsTiled);
							break;
						case 8:
							PaletteCallUnpackIntoRawBufferImpl<DataTypeDest, uint8, TiffImageWrapper::TPaletteAdapter<uint8>>(NumOfChannelDest, bIsTiled);
							break;
						case 16:
							PaletteCallUnpackIntoRawBufferImpl<DataTypeDest, uint16, TiffImageWrapper::TPaletteAdapter<uint16>>(NumOfChannelDest, bIsTiled);
							break;
						case 32:
							PaletteCallUnpackIntoRawBufferImpl<DataTypeDest, uint32, TiffImageWrapper::TPaletteAdapter<uint32>>(NumOfChannelDest, bIsTiled);
							break;

						default:
							SetError(TEXT("Tiff, Unsupported bits per sample from a Palette base image."));
							break;
						}
					}
					else
					{
						SetError(TEXT("Tiff, Unsupported bits per sample from a Palette base image."));
					}
				}
				else if (BitsPerSample == 16)
				{
					DefaultCallUnpackIntoRawBufferImpl<DataTypeDest, uint16>(NumOfChannelDest, bIsTiled);
				}
				else if (BitsPerSample == 32)
				{
					DefaultCallUnpackIntoRawBufferImpl<DataTypeDest, uint32>(NumOfChannelDest, bIsTiled);
				}
				else if (BitsPerSample == 64)
				{
					DefaultCallUnpackIntoRawBufferImpl<DataTypeDest, uint64>(NumOfChannelDest, bIsTiled);
				}
				else
				{
					SetError(TEXT("Tiff, Unsupported bits per sample from a uint sample format."));
				}
			}
			else if (SampleFormat == SAMPLEFORMAT_IEEEFP)
			{
				if (BitsPerSample == 16)
				{
					DefaultCallUnpackIntoRawBufferImpl<DataTypeDest, FFloat16>(NumOfChannelDest, bIsTiled);
				}
				else if (BitsPerSample == 32)
				{
					DefaultCallUnpackIntoRawBufferImpl<DataTypeDest, float>(NumOfChannelDest, bIsTiled);
				}
				else if (BitsPerSample == 64)
				{
					DefaultCallUnpackIntoRawBufferImpl<DataTypeDest, double>(NumOfChannelDest, bIsTiled);
				}
				else
				{
					SetError(TEXT("Tiff, Unsupported bits per sample from a IEEEFP sample format."));
				}
			}
		}
		
		if (Photometric == PHOTOMETRIC_MINISWHITE)
		{
			if constexpr (std::is_same_v<DataTypeDest, FFloat16> || TIsFloatingPoint<DataTypeDest>::Value)
			{
				// miniswhite with floating point tiff is an odd thing to do
				// it's unclear if you should invert against 1.f or something else?
				UE_LOGF(LogImageWrapper, Warning, "Tiff MINISWHITE floating point?  This is probably wrong.");

				TArrayView64<DataTypeDest> FinalImage(static_cast<DataTypeDest*>(static_cast<void*>(SubImageBuffer.GetData())), SubImageBuffer.Num());
				ParallelFor(CurrSubImageWidth* CurrSubImageHeight, [&FinalImage](int32 Index)
					{
						DataTypeDest& FinalValue = FinalImage[Index];
						FinalValue = 1.f - FinalValue; // uses implicit operator float() on FFloat16
					},  IsInGameThread() ? EParallelForFlags::None : EParallelForFlags::BackgroundPriority);
			}
			else
			{
				TArrayView64<DataTypeDest> FinalImage(static_cast<DataTypeDest*>(static_cast<void*>(SubImageBuffer.GetData())), SubImageBuffer.Num());
				ParallelFor(CurrSubImageWidth* CurrSubImageHeight, [&FinalImage](int32 Index)
					{
						DataTypeDest& FinalValue = FinalImage[Index];
						FinalValue = TNumericLimits<DataTypeDest>::Max() - FinalValue;
					},  IsInGameThread() ? EParallelForFlags::None : EParallelForFlags::BackgroundPriority);
			}
		}
	}

	template<class DataTypeDest, class DataTypeSrc, bool bIsTiled, class ReadWriteAdapter>
	bool FTiffImageWrapper::UnpackIntoRawBufferImpl(const uint8 NumOfChannelDest, const bool bAddAlpha)
	{
		using namespace TiffImageWrapper;

		const uint64 LineSizeScr = TIFFScanlineSize64(Tiff);
		const uint64 StripByteSize = TIFFStripSize64(Tiff);

		TArrayView64<DataTypeDest> WriteArray(static_cast<DataTypeDest*>(static_cast<void*>(SubImageBuffer.GetData())), SubImageBuffer.Num() / sizeof(DataTypeDest));

		uint16 PlanarConfig = 0;
		TIFFGetFieldDefaulted(Tiff, TIFFTAG_PLANARCONFIG, &PlanarConfig);
		const bool bIsPlanarConfigSepareted = PlanarConfig == PLANARCONFIG_SEPARATE;

		const uint8 StepDest = bIsPlanarConfigSepareted ? NumOfChannelDest : 1;
		const uint8 NumberOfChannelInReadArray =  bIsPlanarConfigSepareted ? 1 : SamplesPerPixel;

		int32 TileWidth = CurrSubImageWidth;
		int32 TileHeight = CurrSubImageHeight;

		if constexpr (bIsTiled)
		{
			if (!TIFFGetField(Tiff, TIFFTAG_TILEWIDTH, &TileWidth) || !TIFFGetField(Tiff, TIFFTAG_TILELENGTH, &TileHeight))
			{
				SetError(TEXT("The tiff file is invalid.(Tiled image but no tile dimensions)"));
				return false;
			}
		}

		ReadWriteAdapter Adapter(Tiff);

		auto ProcessDecodedData =
				[this, &WriteArray, NumOfChannelDest, StepDest, NumberOfChannelInReadArray, TileWidth, TileHeight, &Adapter]
				(int32 NumberOfColumnRead, int32 NumberOfRowRead, TArray64<uint8>& ReadBuffer, uint8 SampleIndex, int32 BlockX, int32 BlockY)
				{
					TArrayView64<const DataTypeSrc> ReadArray(static_cast<DataTypeSrc*>(static_cast<void*>(ReadBuffer.GetData())), ReadBuffer.Num() / sizeof(DataTypeSrc));
					uint64 CurrentOffset = int64(BlockY) * CurrSubImageWidth * NumOfChannelDest;
					if constexpr (bIsTiled)
					{
						CurrentOffset *= TileHeight;
						CurrentOffset += int64(BlockX) * TileWidth * NumOfChannelDest;
					}
	
					ParallelFor(NumberOfRowRead * NumberOfColumnRead
						, [this, BlockY, &WriteArray, &ReadArray, NumOfChannelDest, StepDest, NumberOfChannelInReadArray, SampleIndex, CurrentOffset, NumberOfColumnRead, TileWidth, &Adapter]
							(int32 PixelReadIndex)
							{
								int64 ReadIndex;
								int64 WriteIndex;
								if constexpr (bIsTiled)
								{
									const int32 PositionXInTile = PixelReadIndex % NumberOfColumnRead;
									const int32 PositionYInTile = PixelReadIndex / NumberOfColumnRead;
									WriteIndex = CurrentOffset + PositionXInTile * NumOfChannelDest + PositionYInTile * CurrSubImageWidth* NumOfChannelDest + SampleIndex;

									// The end of tile in X can be some garbage that act as padding data so that all tiles are the same size
									ReadIndex = PositionXInTile * NumberOfChannelInReadArray + PositionYInTile * TileWidth * NumberOfChannelInReadArray;
								}
								else
								{
									WriteIndex = CurrentOffset + int64(PixelReadIndex) * NumOfChannelDest + SampleIndex;
									ReadIndex = int64(PixelReadIndex) * NumberOfChannelInReadArray;
								}

								for (int32 I = 0; I < NumberOfChannelInReadArray; I++)
								{
									Adapter.ReadAndWrite(ReadArray, ReadIndex, WriteArray, WriteIndex);

									WriteIndex += StepDest;
									++ReadIndex;
								}
							}
						, IsInGameThread() ? EParallelForFlags::None : EParallelForFlags::BackgroundPriority);
				};


		const uint8 NumOfPlanes = bIsPlanarConfigSepareted ? SamplesPerPixel : 1;

		TQueue<TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe>, EQueueMode::Mpsc> UsableBufferQueue;
		FGraphEventArray Tasks;
		
		ENamedThreads::Type NamedThread = IsInGameThread() ? ENamedThreads::GameThread : ENamedThreads::AnyThread;
		ON_SCOPE_EXIT {	FTaskGraphInterface::Get().WaitUntilTasksComplete(Tasks, NamedThread); };

		if constexpr (bIsTiled)
		{
			uint64 TileSize = TIFFTileSize64(Tiff);
			if (TileSize < TileHeight * TileWidth * NumberOfChannelInReadArray * sizeof(DataTypeSrc) / ReadWriteAdapter::ChannelPerReadIndex)
			{
				// Lib tiff and this code is not able to deal with channel of different bit sizes
				SetError(TEXT("Tiff tiles are smaller than expected. This is generally due to channels of different size which is not supported by UE"));
				return false;
			}

			const uint64 NumTiles = TIFFNumberOfTiles(Tiff);
			if (NumTiles > (uint64)INT32_MAX)
			{
				const FString TileErrorMessage = FString::Printf(TEXT("UE can support at most %u tiles at the moment. The file contains %" UINT64_FMT " tiles."), INT32_MAX, NumTiles);
				SetError(*TileErrorMessage);
				return false;
			}

			Tasks.Reserve(NumTiles);

			for (uint8 SampleIndex = 0; SampleIndex < NumOfPlanes; SampleIndex++)
			{ 
				for (int32 Y = 0, TileY = 0; Y < CurrSubImageHeight; Y += TileHeight, ++TileY)
				{
					const int32 NumberOfRow = Y + TileHeight > CurrSubImageHeight ? CurrSubImageHeight - Y : TileHeight;

					for (int32 X = 0, TileX = 0; X < CurrSubImageWidth; X += TileWidth, ++TileX)
					{
						TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> BufferPtr;
						if (!UsableBufferQueue.Dequeue(BufferPtr))
						{
							BufferPtr = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
							BufferPtr->AddUninitialized(TileSize);
						}

						if (TIFFReadTile(Tiff, BufferPtr->GetData(), X, Y, 0, SampleIndex) < 0)
						{
							SetError(TEXT("Couldn't open Tiff tile. This is generally caused by a compression format that UE doesn't support."));
							return false;
						}

						const int32 NumberOfColumn = X + TileWidth > CurrSubImageWidth ? CurrSubImageWidth - X : TileWidth;
						Tasks.Add(TGraphTask<FProcessDecodedDataTask>::CreateTask().ConstructAndDispatchWhenReady([&UsableBufferQueue, &ProcessDecodedData, NumberOfColumn, NumberOfRow, BufferPtr, SampleIndex, TileX, TileY]()
							{
								ProcessDecodedData(NumberOfColumn, NumberOfRow, *(BufferPtr.Get()), SampleIndex, TileX, TileY);
								UsableBufferQueue.Enqueue(BufferPtr);
							}));
					}
				}
			}
		}
		else
		{
			uint32 RowPerStrip = 0;
			TIFFGetField(Tiff, TIFFTAG_ROWSPERSTRIP, &RowPerStrip);

			const int32 NumberOfRow = RowPerStrip > CurrSubImageHeight || RowPerStrip == 0  ? CurrSubImageHeight : RowPerStrip;
			if (StripByteSize < NumberOfRow * CurrSubImageWidth * sizeof(DataTypeSrc) * NumberOfChannelInReadArray / ReadWriteAdapter::ChannelPerReadIndex)
			{
				// Lib tiff and this code is not able to deal with channel of different bit sizes
				SetError(TEXT("Tiff strips are smaller than expected. This is generally due to channels of different size which is not supported by UE"));
				return false;
			}

			const uint64 NumStrips = TIFFNumberOfStrips(Tiff);
			if (NumStrips > (uint64)INT32_MAX)
			{
				const FString StripErrorMessage = FString::Printf(TEXT("UE can support at most %u strips at the moment. The file contains %" UINT64_FMT " strips."), INT32_MAX, NumStrips);
				SetError(*StripErrorMessage);
				return false;
			}

			const TCHAR* ErrorMessage = TEXT("Couldn't open Tiff strip. This is generally caused by a compression format that UE doesn't support.");

			Tasks.Reserve(NumStrips);
			
			for (uint8 SampleIndex = 0; SampleIndex < NumOfPlanes; SampleIndex++)
			{
				if (RowPerStrip != 0)
				{ 
					for (int32 Y = 0; Y < CurrSubImageHeight; Y += RowPerStrip)
					{
						TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> BufferPtr;
						if (!UsableBufferQueue.Dequeue(BufferPtr))
						{
							BufferPtr = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
							BufferPtr->AddUninitialized(StripByteSize);
						}

						// The last strip might be smaller
						int32 NumberOfRowRead = Y + RowPerStrip > CurrSubImageHeight ? CurrSubImageHeight - Y : RowPerStrip;
						if (TIFFReadEncodedStrip(Tiff, TIFFComputeStrip(Tiff, Y, SampleIndex), BufferPtr->GetData(), NumberOfRowRead * LineSizeScr) == -1)
						{
							SetError(ErrorMessage);
							return false;
						}

						
						Tasks.Add(TGraphTask<FProcessDecodedDataTask>::CreateTask().ConstructAndDispatchWhenReady([this, &ProcessDecodedData, &UsableBufferQueue, NumberOfRowRead, BufferPtr, Y, SampleIndex]()
							{
								// Consider the strip to be a tile that as the full with of the image.
								ProcessDecodedData(CurrSubImageWidth, NumberOfRowRead, *(BufferPtr.Get()), SampleIndex, 0, Y);
								UsableBufferQueue.Enqueue(BufferPtr);
							}));
					}
				}
				else
				{
					TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> BufferPtr;
					if (!UsableBufferQueue.Dequeue(BufferPtr))
					{
						BufferPtr = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
						BufferPtr->AddUninitialized(StripByteSize);
					}

					if (TIFFReadEncodedStrip(Tiff, SampleIndex, BufferPtr->GetData(), StripByteSize) == -1)
					{
						SetError(ErrorMessage);
						return false;
					}

					Tasks.Add(TGraphTask<FProcessDecodedDataTask>::CreateTask().ConstructAndDispatchWhenReady([this, &UsableBufferQueue, &ProcessDecodedData, BufferPtr, SampleIndex]()
						{
							// Consider the image to be one big tile.
							ProcessDecodedData(CurrSubImageWidth, CurrSubImageHeight, *(BufferPtr.Get()), SampleIndex, 0, 0);
							UsableBufferQueue.Enqueue(BufferPtr);
						}));
				}
			}
		}

		// Write the alpha
		if (bAddAlpha)
		{
			//Tasks[] is filling WriteArray and not done yet
			//	even if Tasks only writes RGB and we only write A , so there is no race
			//	it's better to not have them running at the same time
			FTaskGraphInterface::Get().WaitUntilTasksComplete(Tasks, NamedThread);
			Tasks.SetNum(0); // clear array so scope exit doesn't wait on tasks again

			// this branch is only used for 3-channel RGB tiffs and 4-channel RGBA output pixels (NumOfChannelDest==4)

			//todo: would be better to do this in the ProcessDecodedData tasks, so that we write the memory only once

			//todo: ParallelFor over individual pixels is not great, better to do on rows; use ImageCore ImageParallelFor
			ParallelFor(CurrSubImageWidth * CurrSubImageHeight, [NumOfChannelDest, &WriteArray](int32 Index)
				{
					const int64 WriteIndex = int64(NumOfChannelDest) * Index + NumOfChannelDest-1;
					if constexpr (std::is_same_v<DataTypeDest, FFloat16> || TIsFloatingPoint<DataTypeDest>::Value)
					{
						WriteArray[WriteIndex] = DataTypeDest(1.f);
					}
					else
					{
						WriteArray[WriteIndex]= TNumericLimits<DataTypeDest>::Max();
					}
				}, IsInGameThread() ? EParallelForFlags::None : EParallelForFlags::BackgroundPriority);
		}

		// scope exit will wait on Tasks[] completing

		return true;
	}

}
#endif // WITH_LIBTIFF