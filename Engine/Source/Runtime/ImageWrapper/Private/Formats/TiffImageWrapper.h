// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageWrapperBase.h"

#if WITH_LIBTIFF

struct tiff;
typedef struct tiff TIFF;

namespace UE::ImageWrapper::Private
{
	class FTiffImageWrapper : public FImageWrapperBase
	{
	public:
		~FTiffImageWrapper();

		FTiffImageWrapper() = default;
		FTiffImageWrapper(FTiffImageWrapper&&) = default;
		FTiffImageWrapper& operator=(FTiffImageWrapper&&) = default;

		FTiffImageWrapper(const FTiffImageWrapper&) = delete;
		FTiffImageWrapper& operator=(const FTiffImageWrapper&) = delete;

		// FImageWrapperBase Interface
		virtual void Compress(int32 Quality) override;
		virtual void Uncompress(const ERGBFormat InFormat, int32 InBitDepth) override;
		virtual void Uncompress(const ERGBFormat InFormat, int32 InBitDepth, FDecompressedImageOutput& OutDecompressedImage) override;
		virtual bool SetCompressed(const void* InCompressedData, int64 InCompressedSize) override;
		
		virtual bool CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const override;
		virtual ERawImageFormat::Type GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const override;

	private:
		bool Uncompress_Internal(const ERGBFormat InFormat, int32 InBitDepth, TMap<FString, FString>& OutTagMetaData);
		
		void ReleaseTiffImage();

		// Unpack the compressed data into the raw buffer. It will also add the alpha channel when needed.
		template<class DataTypeDest>
		void UnpackIntoRawBuffer(const uint8 NumOfChannelDst);

		template<class DataTypeDest, class DataTypeSrc, class Adapter>
		void CallUnpackIntoRawBufferImpl(uint8 NumOfChannelDest, const bool bIsTiled);

		template<class DataTypeDest, class DataTypeSrc>
		void DefaultCallUnpackIntoRawBufferImpl(uint8 NumOfChannelDest, const bool bIsTiled);

		template<class DataTypeDest, class DataTypeSrc, class Adapter>
		void PaletteCallUnpackIntoRawBufferImpl(uint8 NumOfChannelDest, const bool bIsTiled);

		template<class DataTypeDest, class DataTypeSrc, bool bIsTiled, class ReadWriteAdapter>
		bool UnpackIntoRawBufferImpl(const uint8 NumOfChannelDest, const bool bAddAlpha);

		// Postion in the buffer pass as a memory file to LibTiff
		int64 CurrentPosition = 0;

		TIFF* Tiff = nullptr;

		uint16 Photometric = 0;
		uint16 Compression = 0;
		uint16 BitsPerSample = 0;
		uint16 SamplesPerPixel = 0;
		uint16 SampleFormat = 0;

		int64 CurrSubImageWidth = 0;
		int64 CurrSubImageHeight = 0;
		TArray64<uint8> SubImageBuffer;

		friend struct FTIFFReadMemoryFile;
		
		virtual void Reset() override
		{
			FImageWrapperBase::Reset();

			ReleaseTiffImage();
			
			CurrentPosition = 0;
			Photometric = 0;
			Compression = 0;
			BitsPerSample = 0;
			SamplesPerPixel = 0;
			SampleFormat = 0;

			SubImageBuffer.Empty();
		}
	};
}
#endif // WITH_LIBTIFF
