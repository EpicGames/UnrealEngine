// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formats/UEJpegImageWrapper.h"

#include "Async/ParallelFor.h"
#include "Math/Color.h"
#include "Misc/ScopeLock.h"
#include "ImageWrapperPrivate.h"
#include "ImageCoreUtils.h"

#if WITH_UEJPEG

#include "uejpeg.h"

static void* ue_malloc(size_t size) 
{ 
	return FMemory::Malloc(size); 
}

static void ue_free(void* ptr) 
{ 
	FMemory::Free(ptr); 
}

static void* ue_realloc(void* ptr, size_t size) 
{ 
	return FMemory::Realloc(ptr, size); 
}


FUEJpegImageWrapper::FUEJpegImageWrapper(int32 InNumComponents /* = 4 */)
	: FImageWrapperBase()
	, NumComponents(InNumComponents)
{ 
	// NumComponents == 1 for GrayscaleJPEG
	static uejpeg_alloc_t UeAlloc = { ue_malloc, ue_free, ue_realloc };
	uejpeg_set_alloc(&UeAlloc);
}


FUEJpegImageWrapper::~FUEJpegImageWrapper()
{
}

/* FImageWrapperBase interface
 *****************************************************************************/

bool FUEJpegImageWrapper::SetCompressed(const void* InCompressedData, int64 InCompressedSize)
{
	// jpgd doesn't support 64-bit sizes.
	if (InCompressedSize < 0 || InCompressedSize > MAX_uint32)
	{
		return false;
	}

	int x, y, n;
	unsigned char* d;

	// Is it a JPEG?
	if (!FMemory::Memcmp(InCompressedData, "\xFF\xD8\xFF", 3)) 
	{
		// First encode it to a UEJPEG format
		unsigned char* UEJData = 0;
		int UEJDataSize = 0;
		{
			uejpeg_encode_context_t ctx = uejpeg_encode_jpeg_mem_threaded_start((uint8*)InCompressedData, InCompressedSize, 0);	
			if (ctx.error) 
			{
				return false;
			}
			ParallelFor(ctx.jobs_to_run, [&](int idx) {
				uejpeg_encode_jpeg_thread_run(&ctx, idx);
			});
			int err = uejpeg_encode_jpeg_mem_threaded_finish(&ctx, &UEJData, &UEJDataSize);
			if (err) 
			{
				return false;
			}
		}

		// Then decode to a raw image
		{
			uejpeg_decode_context_t ctx = uejpeg_decode_mem_threaded_start(UEJData, UEJDataSize, UEJPEG_FLAG_FASTDCT);
			if (ctx.error)
			{
				FMemory::Free(UEJData);
				return false;
			}
			ParallelFor(ctx.jobs_to_run, [&](int idx) {
				uejpeg_decode_thread_run(&ctx, idx);
			});
			d = uejpeg_decode_threaded_finish(&ctx, &x, &y, &n);
		}
		if (!d)
		{
			FMemory::Free(UEJData);
			return false;
		}
		FMemory::Free(d);

		if (!FImageWrapperBase::SetCompressed(UEJData, UEJDataSize))
		{
			FMemory::Free(UEJData);
			return false;
		}

		FMemory::Free(UEJData);
	}
	else 
	{
		uejpeg_decode_context_t ctx = uejpeg_decode_mem_threaded_start((uint8*)InCompressedData, (uint32)InCompressedSize, UEJPEG_FLAG_FASTDCT);
		if (ctx.error)
		{
			return false;
		}
		ParallelFor(ctx.jobs_to_run, [&](int idx) {
			uejpeg_decode_thread_run(&ctx, idx);
		});
		d = uejpeg_decode_threaded_finish(&ctx, &x, &y, &n);
		if (!d)
		{
			return false;
		}
		FMemory::Free(d);

		if (!FImageWrapperBase::SetCompressed(InCompressedData, InCompressedSize))
		{
			return false;
		}
	}


	// We don't support 16 bit jpegs
	BitDepth = 8;

	Width = x;
	Height = y;

	switch (n)
	{
	case 1:
		Format = ERGBFormat::Gray;
		break;
	case 4:
		Format = ERGBFormat::RGBA;
		break;
	default:
		return false;
	}
	
	if ( ! FImageCoreUtils::IsImageImportPossible(Width,Height) )
	{
		SetError(TEXT("Image dimensions are not possible to import"));
		return false;
	}

	return true;
}


// CanSetRawFormat returns true if SetRaw will accept this format
bool FUEJpegImageWrapper::CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const
{
	return ((InFormat == ERGBFormat::RGBA || InFormat == ERGBFormat::BGRA || InFormat == ERGBFormat::Gray) && InBitDepth == 8);
}

// returns InFormat if supported, else maps to something supported
ERawImageFormat::Type FUEJpegImageWrapper::GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const
{
	switch(InFormat)
	{
	case ERawImageFormat::G8:
	case ERawImageFormat::BGRA8:
		return InFormat; // directly supported
	case ERawImageFormat::BGRE8:
	case ERawImageFormat::RGBA16:
	case ERawImageFormat::RGBA16F:
	case ERawImageFormat::RGBA32F:
	case ERawImageFormat::G16:
	case ERawImageFormat::R16F:
	case ERawImageFormat::R32F:
		return ERawImageFormat::BGRA8; // needs conversion
	default:
		check(0);
		return ERawImageFormat::BGRA8;
	};
}

void FUEJpegImageWrapper::Compress(int32 Quality)
{
	if (Quality == 0)
	{
		//default 
		Quality = 85;
	}
	else if (Quality == (int32)EImageCompressionQuality::Uncompressed)
	{
		// fix = 1 (Uncompressed) was previously treated as max-compress
		Quality = 100;
	}
	else
	{
		ensure(Quality >= 1 && Quality <= 100);

		#define JPEG_QUALITY_MIN 40
		// JPEG should not be used below quality JPEG_QUALITY_MIN
		Quality = FMath::Clamp(Quality, JPEG_QUALITY_MIN, 100);
	}

	if (CompressedData.Num() == 0)
	{
		check(RawData.Num());
		check(Width > 0);
		check(Height > 0);

		// re-order components if required - JPEGs expect RGBA
		if(Format == ERGBFormat::BGRA)
		{
			FColor* Colors = (FColor*)RawData.GetData();
			const int32 NumColors = RawData.Num() / 4;
			for(int32 ColorIndex = 0; ColorIndex < NumColors; ColorIndex++)
			{
				int32 tmp = Colors[ColorIndex].B;
				Colors[ColorIndex].B = Colors[ColorIndex].R;
				Colors[ColorIndex].R = tmp;
			}
		}


		int data_size;
		unsigned char* data;
		uejpeg_encode_image_context_t ctx = uejpeg_encode_image_mem_threaded_start(Width, Height, NumComponents, RawData.GetData(), Quality, 0);
		check(!ctx.error);
		ParallelFor(ctx.jobs_to_run, [&](int idx) {
			uejpeg_encode_image_thread_run(&ctx, idx);
		});
		int error = uejpeg_encode_image_mem_threaded_finish(&ctx, &data, &data_size);
		check(!error);

		CompressedData.Reset(data_size);
		CompressedData.AddUninitialized(data_size);
		FMemory::Memcpy(CompressedData.GetData(), data, data_size);
		FMemory::Free(data);
	}
}


void FUEJpegImageWrapper::Uncompress(const ERGBFormat InFormat, int32 InBitDepth)
{
	// Ensure we haven't already uncompressed the file.
	if (RawData.Num() != 0)
	{
		return;
	}

	check(CompressedData.Num());

	int32 NumColors;
	int32 jpegWidth,jpegHeight;
	uint8* OutData;
	uejpeg_decode_context_t ctx = uejpeg_decode_mem_threaded_start(CompressedData.GetData(), CompressedData.Num(), UEJPEG_FLAG_FASTDCT);
	if (ctx.error)
	{
		return;
	}
	ParallelFor(ctx.jobs_to_run, [&](int idx) {
		uejpeg_decode_thread_run(&ctx, idx);
	});
	OutData = uejpeg_decode_threaded_finish(&ctx, &jpegWidth, &jpegHeight, &NumColors);

	if (OutData)
	{
		Width = jpegWidth;
		Height = jpegHeight;
		RawData.Reset(Width * Height * NumColors);
		RawData.AddUninitialized(Width * Height * NumColors);
		FMemory::Memcpy(RawData.GetData(), OutData, RawData.Num());
		FMemory::Free(OutData);
	}
	else
	{
		UE_LOGF(LogImageWrapper, Error, "UEJPEG Decompress Error");
		RawData.Empty();
	}
}

TArray64<uint8> FUEJpegImageWrapper::GetExportData(int32 Quality)
{
	uint8* Data;
	int Size;
	uejpeg_decode_mem_to_jpeg(CompressedData.GetData(), CompressedData.Num(), &Data, &Size, 0);

	TArray64<uint8> ExportData;
	ExportData.Reset();
	ExportData.AddUninitialized(Size);
	FMemory::Memcpy(ExportData.GetData(), Data, Size);
	FMemory::Free(Data);
	return ExportData;
}

#endif //WITH_UEJPEG
