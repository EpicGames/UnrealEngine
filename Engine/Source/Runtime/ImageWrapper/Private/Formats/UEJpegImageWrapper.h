// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ImageWrapperBase.h"

#if WITH_UEJPEG

class FUEJpegImageWrapper
	: public FImageWrapperBase
{
public:

	/** Default constructor. */
	// NumComponents == 1 for GrayscaleJPEG
	FUEJpegImageWrapper(int32 InNumComponents = 4);

	virtual ~FUEJpegImageWrapper();

public:

	//~ FImageWrapperBase interface

	virtual bool SetCompressed(const void* InCompressedData, int64 InCompressedSize) override;
	virtual void Uncompress(const ERGBFormat InFormat, int32 InBitDepth) override;
	virtual void Compress(int32 Quality) override;
	virtual TArray64<uint8> GetExportData(int32 Quality) override;
	
	virtual bool CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const override;
	virtual ERawImageFormat::Type GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const override;

private:

	int32 NumComponents;
};


#endif //WITH_UEJPEG
