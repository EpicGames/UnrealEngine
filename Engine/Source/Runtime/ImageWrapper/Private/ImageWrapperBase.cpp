// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageWrapperBase.h"
#include "ImageWrapperPrivate.h"
#include "ImageWrapperOutputTypes.h"


/* FImageWrapperBase structors
 *****************************************************************************/

FImageWrapperBase::FImageWrapperBase()
	: Format(ERGBFormat::Invalid)
	, BitDepth(0)
	, Width(0)
	, Height(0)
{ }


/* FImageWrapperBase interface
 *****************************************************************************/

void FImageWrapperBase::Reset()
{
	LastError.Empty();
	
	RawData.Empty();
	CompressedData.Empty();

	Format = ERGBFormat::Invalid;
	BitDepth = 0;
	Width = 0;
	Height = 0;
}


void FImageWrapperBase::SetError(const TCHAR* ErrorMessage)
{
	LastError = ErrorMessage;
}


/* IImageWrapper structors
 *****************************************************************************/

void FImageWrapperBase::Uncompress(const ERGBFormat InFormat, int32 InBitDepth, FDecompressedImageOutput& OutDecompressedImage)
{
	OutDecompressedImage = FDecompressedImageOutput();
	Uncompress(InFormat, InBitDepth);
	OutDecompressedImage.MipMapImage.AddMipImage(MoveTemp(RawData), GetWidth(), GetHeight());
}

TArray64<uint8> FImageWrapperBase::GetCompressed(int32 Quality)
{
	LastError.Empty();
	Compress(Quality);

	return MoveTemp(CompressedData);
}


bool FImageWrapperBase::GetRaw(const ERGBFormat InFormat, int32 InBitDepth, TArray64<uint8>& OutRawData)
{
	LastError.Empty();
	Uncompress(InFormat, InBitDepth);

	if (!LastError.IsEmpty())
	{
		UE_LOGF(LogImageWrapper, Warning, "ImageWrapper GetRaw failed: %ls", *LastError);
		return false;
	}

	if (RawData.IsEmpty())
	{
		return false;
	}

	OutRawData = MoveTemp(RawData);

	return true;
}

bool FImageWrapperBase::GetRaw(const ERGBFormat InFormat, int32 InBitDepth, FDecompressedImageOutput& OutDecompressedImage)
{
	LastError.Empty();

	// Uncompress should add data directly to the mip map image through FMipMapImage::AddMipImage
	Uncompress(InFormat, InBitDepth, OutDecompressedImage);

	if (!LastError.IsEmpty())
	{
		UE_LOGF(LogImageWrapper, Warning, "ImageWrapper GetRaw failed: %ls", *LastError);
		return false;
	}

	if (OutDecompressedImage.MipMapImage.RawData.IsEmpty())
	{
		return false;
	}

	return true;
}


bool FImageWrapperBase::SetCompressed(const void* InCompressedData, int64 InCompressedSize)
{
	Reset();
	RawData.Empty();			// Invalidates the raw data too

	if(InCompressedSize > 0 && InCompressedData != nullptr)
	{
		// this is usually an unnecessary allocation and copy
		// we should decompress directly from the source buffer

		CompressedData.Empty(InCompressedSize);
		CompressedData.AddUninitialized(InCompressedSize);
		FMemory::Memcpy(CompressedData.GetData(), InCompressedData, InCompressedSize);

		return true;
	}

	return false;
}


bool FImageWrapperBase::SetRaw(const void* InRawData, int64 InRawSize, const int32 InWidth, const int32 InHeight, const ERGBFormat InFormat, const int32 InBitDepth, const int32 InBytesPerRow)
{
	#define check_and_return_false(exp)		do { check(exp); if ( ! (exp) ) return false; } while(0)

	check_and_return_false(InRawData != NULL);
	check_and_return_false(InRawSize > 0);
	check_and_return_false(InWidth > 0);
	check_and_return_false(InHeight > 0);
	check_and_return_false(InBytesPerRow >= 0);

	Reset();
	CompressedData.Empty();		// Invalidates the compressed data too
	
	if ( ! CanSetRawFormat(InFormat,InBitDepth) )
	{
		UE_LOGF(LogImageWrapper, Warning, "ImageWrapper unsupported format; check CanSetRawFormat; %d x %d", (int)InFormat,InBitDepth);
		return false;
	}
	
	Format = InFormat;
	BitDepth = InBitDepth;
	Width = InWidth;
	Height = InHeight;

	int64 BytesPerRow = GetBytesPerRow();
	
	int64 RawSize = BytesPerRow * Height;
	RawData.Empty(RawSize);
	RawData.AddUninitialized(RawSize);

	// copy the incoming data directly
	if ( InBytesPerRow == 0 || BytesPerRow == InBytesPerRow )
	{
		// this is usually an unnecessary allocation and copy
		// we should compress directly from the source buffer

		check_and_return_false( InRawSize >= RawSize );

		FMemory::Memcpy(RawData.GetData(), InRawData, RawSize);
	}
	else
	{
		// The supported image formats (PNG, BMP, etc) don't support strided data (although turbo jpeg does).
		// Therefore we de-stride the data here so we can uniformly support strided input data with all the supported
		// image formats.
		check_and_return_false(InBytesPerRow > BytesPerRow); // equality handled in above branch
		check_and_return_false(InRawSize >= (int64) (Height-1)*InBytesPerRow + BytesPerRow);

		for (int32 y = 0; y < Height; y++)
		{
			FMemory::Memcpy(RawData.GetData() + (y * BytesPerRow), (uint8*)InRawData + (int64) y * InBytesPerRow, BytesPerRow);
		}
	}
	
	#undef check_and_return_false

	return true;
}


int64 IImageWrapper::GetRGBFormatBytesPerPel(ERGBFormat RGBFormat,int BitDepth)
{
	switch(RGBFormat)
	{
	case ERGBFormat::RGBA:
	case ERGBFormat::BGRA:
		if ( BitDepth == 8 )
		{
			return 4;
		}
		else if ( BitDepth == 16 )
		{
			return 8;
		}
		break;
						
	case ERGBFormat::Gray:
		if ( BitDepth == 8 )
		{
			return 1;
		}
		else if ( BitDepth == 16 )
		{
			return 2;
		}
		break;
			
	case ERGBFormat::BGRE:
		if ( BitDepth == 8 )
		{
			return 4;
		}
		break;
			
	case ERGBFormat::RGBAF:
		if ( BitDepth == 16 )
		{
			return 8;
		}
		else if ( BitDepth == 32 )
		{
			return 16;
		}
		break;
			
	case ERGBFormat::GrayF:
		if ( BitDepth == 16 )
		{
			return 2;
		}
		else if ( BitDepth == 32 )
		{
			return 4;
		}
		break;

	default:
		break;
	}

	UE_LOGF(LogImageWrapper, Error, "GetRGBFormatBytesPerPel not handled : %d,%d", (int)RGBFormat,BitDepth );
	return 0;
}


ERawImageFormat::Type IImageWrapper::ConvertRGBFormat(ERGBFormat RGBFormat,int BitDepth,bool * bIsExactMatch)
{
	bool bIsExactMatchDummy;
	if ( ! bIsExactMatch )
	{
		bIsExactMatch = &bIsExactMatchDummy;
	}
		
	switch(RGBFormat)
	{
	case ERGBFormat::RGBA:
		if ( BitDepth == 8 )
		{
			*bIsExactMatch = false; // needs RB swap
			return ERawImageFormat::BGRA8;
		}
		else if ( BitDepth == 16 )
		{
			*bIsExactMatch = true;
			return ERawImageFormat::RGBA16;
		}
		break;
			
	case ERGBFormat::BGRA:
		if ( BitDepth == 8 )
		{
			*bIsExactMatch = true;
			return ERawImageFormat::BGRA8;
		}
		else if ( BitDepth == 16 )
		{
			*bIsExactMatch = false; // needs RB swap
			return ERawImageFormat::RGBA16;
		}
		break;
			
	case ERGBFormat::Gray:
		if ( BitDepth == 8 )
		{
			*bIsExactMatch = true;
			return ERawImageFormat::G8;
		}
		else if ( BitDepth == 16 )
		{
			*bIsExactMatch = true;
			return ERawImageFormat::G16;
		}
		break;
			
	case ERGBFormat::BGRE:
		if ( BitDepth == 8 )
		{
			*bIsExactMatch = true;
			return ERawImageFormat::BGRE8;
		}
		break;
			
	case ERGBFormat::RGBAF:
		if ( BitDepth == 16 )
		{
			*bIsExactMatch = true;
			return ERawImageFormat::RGBA16F;
		}
		else if ( BitDepth == 32 )
		{
			*bIsExactMatch = true;
			return ERawImageFormat::RGBA32F;
		}
		break;
			
	case ERGBFormat::GrayF:
		if ( BitDepth == 16 )
		{
			*bIsExactMatch = true;
			return ERawImageFormat::R16F;
		}
		else if ( BitDepth == 32 )
		{
			*bIsExactMatch = true;
			return ERawImageFormat::R32F;
		}
		break;

	default:
		break;
	}

	UE_LOGF(LogImageWrapper, Warning, "ConvertRGBFormat not handled : %d,%d", (int)RGBFormat,BitDepth );
		
	*bIsExactMatch = false;
	return ERawImageFormat::Invalid;
}
	
void IImageWrapper::ConvertRawImageFormat(ERawImageFormat::Type RawFormat, ERGBFormat & OutFormat,int & OutBitDepth)
{
	switch(RawFormat)
	{
	case ERawImageFormat::G8:
		OutFormat = ERGBFormat::Gray;
		OutBitDepth = 8;
		break;
	case ERawImageFormat::BGRA8:
		OutFormat = ERGBFormat::BGRA;
		OutBitDepth = 8;
		break;
	case ERawImageFormat::BGRE8:
		OutFormat = ERGBFormat::BGRE;
		OutBitDepth = 8;
		break;
	case ERawImageFormat::RGBA16:
		OutFormat = ERGBFormat::RGBA;
		OutBitDepth = 16;
		break;
	case ERawImageFormat::RGBA16F:
		OutFormat = ERGBFormat::RGBAF;
		OutBitDepth = 16;
		break;
	case ERawImageFormat::RGBA32F:
		OutFormat = ERGBFormat::RGBAF;
		OutBitDepth = 32;
		break;
	case ERawImageFormat::G16:
		OutFormat = ERGBFormat::Gray;
		OutBitDepth = 16;
		break;
	case ERawImageFormat::R16F:
		OutFormat = ERGBFormat::GrayF;
		OutBitDepth = 16;
		break;
	case ERawImageFormat::R32F:
		OutFormat = ERGBFormat::GrayF;
		OutBitDepth = 32;
		break;
	default:
		check(0);
		break;
	}
}

bool FImageWrapperBase::GetImageViewOfSetRawForCompress(FImageView & OutImage) const
{
	if ( RawData.IsEmpty() )
	{
		return false;
	}
	
	bool bExactMatch;
	ERawImageFormat::Type RawFormat = GetClosestRawImageFormat(&bExactMatch);
	
	// must be bExactMatch, no RB swaps possible, because we just point at the array
	//	this function will fail if SetRaw is done with RGBA8 rather than BGRA8
	if ( RawFormat == ERawImageFormat::Invalid || !bExactMatch )
	{
		return false;
	}

	// ImageWrapper RGBFormat doesn't track if pixels are Gamma/sRGB or not
	//	just assume they are Default for now :
	EGammaSpace GammaSpace = ERawImageFormat::GetDefaultGammaSpace(RawFormat);
	
	OutImage.RawData = (void *) &RawData[0];
	OutImage.SizeX = Width;
	OutImage.SizeY = Height;
	OutImage.NumSlices = 1;
	OutImage.Format = RawFormat;
	OutImage.GammaSpace = GammaSpace;

	return true;
}

bool IImageWrapper::GetRawImage(FImage & OutImage)
{
	TArray64<uint8> OutRawData;
	if (!GetRaw(OutRawData))
	{
		return false;
	}

	int64 Width = GetWidth();
	int64 Height = GetHeight();
	ERGBFormat RGBFormat = GetFormat();
	int32 BitDepth = GetBitDepth();

	bool bExactMatch;
	ERawImageFormat::Type RawFormat = GetClosestRawImageFormat(&bExactMatch);

	if (RawFormat == ERawImageFormat::Invalid)
	{
		return false;
	}

	// ImageWrapper RGBFormat doesn't track if pixels are Gamma/sRGB or not
	//	just assume they are Default for now :
	EGammaSpace GammaSpace = ERawImageFormat::GetDefaultGammaSpace(RawFormat);

	if (bExactMatch)
	{
		// no conversion required

		OutImage.RawData = MoveTemp(OutRawData);
		OutImage.SizeX = Width;
		OutImage.SizeY = Height;
		OutImage.NumSlices = 1;
		OutImage.Format = RawFormat;
		OutImage.GammaSpace = GammaSpace;
	}
	else
	{
		OutImage.Init(Width, Height, RawFormat, GammaSpace);

		FImageView SrcImage = OutImage;
		SrcImage.RawData = OutRawData.GetData();

		switch (RGBFormat)
		{
		case ERGBFormat::RGBA:
		{
			// RGBA8 -> BGRA8
			check(BitDepth == 8);
			check(RawFormat == ERawImageFormat::BGRA8);
			FImageCore::CopyImageRGBABGRA(SrcImage, OutImage);
			break;
		}

		case ERGBFormat::BGRA:
		{
			// BGRA16 -> RGBA16
			check(BitDepth == 16);
			check(RawFormat == ERawImageFormat::RGBA16);
			FImageCore::CopyImageRGBABGRA(SrcImage, OutImage);
			break;
		}

		default:
			check(0);
			return false;
		}
	}

	return true;
}

bool IImageWrapper::GetRawImage(FDecompressedImageOutput& OutDecomressedImage)
{
	
	if (!GetRaw(OutDecomressedImage))
	{
		return false;
	}

	int64 Width = GetWidth();
	int64 Height = GetHeight();
	ERGBFormat RGBFormat = GetFormat();
	int32 BitDepth = GetBitDepth();

	bool bExactMatch;
	ERawImageFormat::Type RawFormat = GetClosestRawImageFormat(&bExactMatch);

	if (RawFormat == ERawImageFormat::Invalid)
	{
		return false;
	}

	// ImageWrapper RGBFormat doesn't track if pixels are Gamma/sRGB or not
	//	just assume they are Default for now :
	EGammaSpace GammaSpace = ERawImageFormat::GetDefaultGammaSpace(RawFormat);

	if (bExactMatch)
	{
		// no conversion required
		OutDecomressedImage.MipMapImage.Format = RawFormat;
		OutDecomressedImage.MipMapImage.GammaSpace = GammaSpace;
	}
	else
	{
		FMipMapImage& MipMapImage = OutDecomressedImage.MipMapImage;

		for (int32 MipLevelIndex = 0; MipLevelIndex < MipMapImage.GetMipCount(); ++MipLevelIndex)
		{
			FImageView SrcImage = MipMapImage.GetMipImage(MipLevelIndex);

			switch (RGBFormat)
			{
			case ERGBFormat::RGBA:
			{
				// RGBA8 -> BGRA8
				check(BitDepth == 8);
				check(RawFormat == ERawImageFormat::BGRA8);
				FImageCore::TransposeImageRGBABGRA(SrcImage);
				break;
			}

			case ERGBFormat::BGRA:
			{
				// BGRA16 -> RGBA16
				check(BitDepth == 16);
				check(RawFormat == ERawImageFormat::RGBA16);
				FImageCore::TransposeImageRGBABGRA(SrcImage);
				break;
			}

			default:
				check(0);
				return false;
			}
		}
	}

	return true;
}

bool FImageWrapperBase::SupportsMetadata() const
{
	return false;
}

void FImageWrapperBase::AddMetadata(const FString&, const FString&)
{
}

void FImageWrapperBase::GetMetadata(TStringMap& OutMetadata) const
{
	OutMetadata.Reset();
}