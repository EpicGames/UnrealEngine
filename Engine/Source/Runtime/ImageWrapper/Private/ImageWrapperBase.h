// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "IImageWrapper.h"


/**
 * The abstract helper class for handling the different image formats
 */
class FImageWrapperBase
	: public IImageWrapper
{
public:

	/** Default Constructor. */
	FImageWrapperBase();

public:

	/**
	 * Gets the image's raw data.
	 *
	 * @return A read-only byte array containing the data.
	 */
	const TArray64<uint8>& GetRawData() const
	{
		return RawData;
	}

	/**
	 * Moves the image's raw data into the provided array.
	 *
	 * @param OutRawData The destination array.
	 */
	void MoveRawData(TArray64<uint8>& OutRawData)
	{
		OutRawData = MoveTemp(RawData);
	}

public:

	/**
	 * Compresses the data.
	 *
	 * @param Quality The compression quality.
	 * 
	 * returns void. call SetError() in your implementation if you fail.
	 */
	virtual void Compress(int32 Quality) = 0;

	/**
	 * Resets the local variables.
	 */
	virtual void Reset();

	/**
	 * Sets last error message.
	 *
	 * @param ErrorMessage The error message to set.
	 */
	void SetError(const TCHAR* ErrorMessage);
	
	/**
	 * Gets last error message.
	 */
	const FString & GetLastError() const
	{
		return LastError;
	}

	/**  
	 * Function to uncompress our data 
	 *
	 * @param InFormat How we want to manipulate the RGB data
	 * @param InBitDepth The bit depth per-channel of the image.
	 * 
	 * returns void. call SetError() in your implementation if you fail.
	 */
	virtual void Uncompress(const ERGBFormat InFormat, int32 InBitDepth) = 0;

	/**
	 * Function to uncompress our data and retrive meta info along with it.
	 *
	 * @param InFormat				How we want to manipulate the RGB data
	 * @param InBitDepth			The bit depth per-channel of the image.
	 * @param OutDecompressedImage	Holds decompressed image, its mip-maps if available along with meta 
	 *								data that can be retrived during the decompression of the image.
	 * 
	 * returns void. call SetError() in your implementation if you fail.
	 */
	virtual void Uncompress(const ERGBFormat InFormat, int32 InBitDepth, FDecompressedImageOutput& OutDecompressedImage);

public:

	//~ IImageWrapper interface

	virtual TArray64<uint8> GetCompressed(int32 Quality = 0) override;

	virtual int32 GetBitDepth() const override
	{
		return BitDepth;
	}

	virtual ERGBFormat GetFormat() const override
	{
		return Format;
	}

	virtual int64 GetHeight() const override
	{
		return Height;
	}

	virtual bool GetRaw(const ERGBFormat InFormat, int32 InBitDepth, TArray64<uint8>& OutRawData) override;

	// Use this version when you expect more information such as mipmaps, etc. 
	virtual bool GetRaw(const ERGBFormat InFormat, int32 InBitDepth, FDecompressedImageOutput& OutDecompressedImage) override;

	virtual int64 GetWidth() const override
	{
		return Width;
	}

	virtual bool SetCompressed(const void* InCompressedData, int64 InCompressedSize) override;
	virtual bool SetRaw(const void* InRawData, int64 InRawSize, const int32 InWidth, const int32 InHeight, const ERGBFormat InFormat, const int32 InBitDepth, const int32 InBytesPerRow = 0) override;

	/**
	 * Does this image type support embedded metadata in its header?
	 *
	 * PNG is an example of an image type which supports adding user-defined metadata to its header.
	 */
	virtual bool SupportsMetadata() const override;

	/**
	 * Adds a key and value to this image's metadata.  Will be saved in the image's header and restored when the image is loaded.
	 *
	 * @param InKey Metadata consists of key value pairs.
	 * @param InValue Metadata consists of key value pairs.
	 */
	virtual void AddMetadata(const FString& InKey, const FString& InValue) override;

	/**
	 * Copies this image's metadata to the caller's map.
	 *
	 * @param OutMetadata A copy of this image's metadata.
	 */
	virtual void GetMetadata(TStringMap& OutMetadata) const override;

protected:

	int64 GetBytesPerPel() const { return GetRGBFormatBytesPerPel(Format,BitDepth); }
	int64 GetBytesPerRow() const { return Width * GetBytesPerPel(); }

	// For writers: after SetRaw(), call this to get an ImageView of the raw data that was set
	//	can return false if the SetRaw does not map to an image format
	//	pixels point at the RawData array
	bool GetImageViewOfSetRawForCompress(FImageView & OutImage) const;

	/** Arrays of compressed/raw data */
	TArray64<uint8> RawData;
	TArray64<uint8> CompressedData;

	/** Format of the raw data */
	ERGBFormat Format;
	int BitDepth;

	/** Width/Height of the image data */
	int64 Width;
	int64 Height;
	
	/** Last Error Message. */
	FString LastError;
};
