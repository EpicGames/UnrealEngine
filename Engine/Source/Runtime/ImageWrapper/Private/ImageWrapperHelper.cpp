// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageWrapperHelper.h"
#include "Misc/Paths.h"

FStringView ImageWrapperHelper::GetFormatExtension(EImageFormat InImageFormat, bool bIncludeDot)
{
	const TCHAR* StringExtension = nullptr;
	switch (InImageFormat)
	{
	case EImageFormat::Invalid:
		return FStringView(TEXT(""));
	case EImageFormat::PNG:
		StringExtension = TEXT(".png");
		break;
	case EImageFormat::JPEG:
	case EImageFormat::GrayscaleJPEG:
		StringExtension = TEXT(".jpg");
		break;
	case EImageFormat::UEJPEG:
	case EImageFormat::GrayscaleUEJPEG:
		StringExtension = TEXT(".uej");
		break;
	case EImageFormat::BMP:
		StringExtension = TEXT(".bmp");
		break;
	case EImageFormat::ICO:
		StringExtension = TEXT(".ico");
		break;
	case EImageFormat::EXR:
		StringExtension = TEXT(".exr");
		break;
	case EImageFormat::ICNS:
		StringExtension = TEXT(".icns");
		break;
	case EImageFormat::TGA:
		StringExtension = TEXT(".tga");
		break;
	case EImageFormat::HDR:
		StringExtension = TEXT(".hdr");
		break;
	default:
		// this should log error
		return FStringView(TEXT(""));
	}
	if (!bIncludeDot) 
	{
		++StringExtension;
	}
	return StringExtension;
}

EImageFormat ImageWrapperHelper::GetImageFormat(FStringView StringExtention)
{
	// deprecated, use ImageWrapper::GetImageFormatFromExtension instead

	int32 Length = StringExtention.Len();
	if (Length == 0)
	{
		return EImageFormat::Invalid;
	}

	if (StringExtention.GetData()[0] == TEXT('.')) 
	{
		StringExtention = StringExtention.SubStr(1, Length);
	}

	if (StringExtention.Equals(TEXT("png"), ESearchCase::IgnoreCase))
	{
		return EImageFormat::PNG;
	}
	if (StringExtention.Equals(TEXT("jpeg"), ESearchCase::IgnoreCase))
	{
		return EImageFormat::JPEG;
	}
	if (StringExtention.Equals(TEXT("jpg"), ESearchCase::IgnoreCase))
	{
		return EImageFormat::JPEG;
	}
	if (StringExtention.Equals(TEXT("uej"), ESearchCase::IgnoreCase))
	{
		return EImageFormat::UEJPEG;
	}
	if (StringExtention.Equals(TEXT("bmp"), ESearchCase::IgnoreCase))
	{
		return EImageFormat::BMP;
	}
	if (StringExtention.Equals(TEXT("ico"), ESearchCase::IgnoreCase))
	{
		return EImageFormat::ICO;
	}
	if (StringExtention.Equals(TEXT("exr"), ESearchCase::IgnoreCase))
	{
		return EImageFormat::EXR;
	}
	if (StringExtention.Equals(TEXT("icns"), ESearchCase::IgnoreCase))
	{
		return EImageFormat::ICNS;
	}
	if (StringExtention.Equals(TEXT("tga"), ESearchCase::IgnoreCase))
	{
		return EImageFormat::TGA;
	}
	if (StringExtention.Equals(TEXT("hdr"), ESearchCase::IgnoreCase))
	{
		return EImageFormat::HDR;
	}
	return EImageFormat::Invalid;
}

const FStringView ImageWrapperHelper::GetImageFilesFilterString(bool bIncludeAllFiles)
{
	return (bIncludeAllFiles)
		? TEXT("Image files (*.jpg; *.png; *.bmp; *.ico; *.exr; *.icns; *.jpeg; *.tga; *.hdr; *.ooj)|*.jpg; *.png; *.bmp; *.ico; *.exr; *.icns; *.jpeg; *.tga; *.hdr; *.ooj|All files (*.*)|*.*")
		: TEXT("Image files (*.jpg; *.png; *.bmp; *.ico; *.exr; *.icns; *.jpeg; *.tga; *.hdr; *.ooj)|*.jpg; *.png; *.bmp; *.ico; *.exr; *.icns; *.jpeg; *.tga; *.hdr; *.ooj");
}