// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageWrapperPrivate.h"

#include "CoreTypes.h"
#include "Modules/ModuleManager.h"

#include "Formats/BmpImageWrapper.h"
#include "Formats/ExrImageWrapper.h"
#include "Formats/HdrImageWrapper.h"
#include "Formats/IcnsImageWrapper.h"
#include "Formats/IcoImageWrapper.h"
#include "Formats/JpegImageWrapper.h"
#include "Formats/UEJpegImageWrapper.h"
#include "Formats/PngImageWrapper.h"
#include "Formats/TgaImageWrapper.h"
#include "Formats/TiffImageWrapper.h"
#include "Formats/DdsImageWrapper.h"

#include "DDSFile.h"

#include "IImageWrapperModule.h"


DEFINE_LOG_CATEGORY(LogImageWrapper);


namespace
{
	static const uint8 IMAGE_MAGIC_PNG[]  = {0x89, 0x50, 0x4E, 0x47, 0x0D, 0x0A, 0x1A, 0x0A};
	static const uint8 IMAGE_MAGIC_JPEG[] = {0xFF, 0xD8, 0xFF};
	static const uint8 IMAGE_MAGIC_BMP[]  = {0x42, 0x4D};
	static const uint8 IMAGE_MAGIC_ICO[]  = {0x00, 0x00, 0x01, 0x00};
	static const uint8 IMAGE_MAGIC_EXR[]  = {0x76, 0x2F, 0x31, 0x01};
	static const uint8 IMAGE_MAGIC_ICNS[] = {0x69, 0x63, 0x6E, 0x73};
	static const uint8 IMAGE_MAGIC_UEJPEG[] = {'O', 'O', 'J', 'P', 'E', 'G'};

	// Binary for #?RADIANCE
	static const uint8 IMAGE_MAGIC_HDR[] = {0x23, 0x3f, 0x52, 0x41, 0x44, 0x49, 0x41, 0x4e, 0x43, 0x45, 0x0a};
	
	// Binary for #?RGBE
	static const uint8 IMAGE_MAGIC_HDR2[] = {0x23, 0x3f, 0x52, 0x47, 0x42, 0x45 };

	// Tiff has two magic bytes sequence
	static const uint8 IMAGE_MAGIC_TIFF_LITTLE_ENDIAN[] = {0x49, 0x49, 0x2A, 0x00};
	static const uint8 IMAGE_MAGIC_TIFF_BIG_ENDIAN[] = {0x4D, 0x4D, 0x00, 0x2A};

	/** Internal helper function to verify image signature. */
	template <int32 MagicCount> bool StartsWith(const uint8* Content, int64 ContentSize, const uint8 (&Magic)[MagicCount])
	{
		if (ContentSize < MagicCount)
		{
			return false;
		}

		for (int32 I = 0; I < MagicCount; ++I)
		{
			if (Content[I] != Magic[I])
			{
				return false;
			}
		}

		return true;
	}
}


/**
 * Image Wrapper module.
 */
class FImageWrapperModule
	: public IImageWrapperModule
{
public:

	//~ IImageWrapperModule interface

	virtual TSharedPtr<IImageWrapper> CreateImageWrapper(const EImageFormat InFormat, const TCHAR* InOptionalDebugImageName = nullptr) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ImageWrapper.Create);

		TSharedPtr<IImageWrapper> ImageWrapper;

		// Allocate a helper for the format type
		switch(InFormat)
		{
#if WITH_UNREALPNG
		case EImageFormat::PNG:
			ImageWrapper = MakeShared<FPngImageWrapper>();
			break;
#endif	// WITH_UNREALPNG

#if WITH_UNREALJPEG
		case EImageFormat::JPEG:
			ImageWrapper = MakeShared<FJpegImageWrapper>();
			break;

		case EImageFormat::GrayscaleJPEG:
			ImageWrapper = MakeShared<FJpegImageWrapper>(1);
			break;
#endif	//WITH_UNREALJPEG

#if WITH_UEJPEG
		case EImageFormat::UEJPEG:
			ImageWrapper = MakeShared<FUEJpegImageWrapper>();
			break;

		case EImageFormat::GrayscaleUEJPEG:
			ImageWrapper = MakeShared<FUEJpegImageWrapper>(1);
			break;
#endif // WITH_UEJPEG

		case EImageFormat::BMP:
			ImageWrapper = MakeShared<FBmpImageWrapper>();
			break;

		case EImageFormat::ICO:
			ImageWrapper = MakeShared<FIcoImageWrapper>();
			break;

#if WITH_UNREALEXR || WITH_UNREALEXR_MINIMAL
		case EImageFormat::EXR:
			ImageWrapper = MakeShared<FExrImageWrapper>();
			break;
#endif
		case EImageFormat::ICNS:
			ImageWrapper = MakeShared<FIcnsImageWrapper>();
			break;

		case EImageFormat::TGA:
			ImageWrapper = MakeShared<FTgaImageWrapper>();
			break;
		case EImageFormat::HDR:
			ImageWrapper = MakeShared<FHdrImageWrapper>();
			break;

#if WITH_LIBTIFF
		case EImageFormat::TIFF:
			ImageWrapper = MakeShared<UE::ImageWrapper::Private::FTiffImageWrapper>();
			break;
#endif // WITH_LIBTIFF

		case EImageFormat::DDS:
			ImageWrapper = MakeShared<FDdsImageWrapper>();
			break;

		default:
			break;
		}

		if (InOptionalDebugImageName != nullptr)
		{
			ImageWrapper->SetDebugImageName(InOptionalDebugImageName);
		}

		return ImageWrapper;
	}

	virtual EImageFormat DetectImageFormat(const void* CompressedData, int64 CompressedSize) override
	{
		EImageFormat Format = EImageFormat::Invalid;
		if (StartsWith((uint8*) CompressedData, CompressedSize, IMAGE_MAGIC_PNG))
		{
			Format = EImageFormat::PNG;
		}
		else if (StartsWith((uint8*) CompressedData, CompressedSize, IMAGE_MAGIC_JPEG))
		{
			Format = EImageFormat::JPEG; // @Todo: Should we detect grayscale vs non-grayscale?
		}
		else if (StartsWith((uint8*)CompressedData, CompressedSize, IMAGE_MAGIC_UEJPEG))
		{
			Format = EImageFormat::UEJPEG; // @Todo: Should we detect grayscale vs non-grayscale?
		}
		else if (StartsWith((uint8*) CompressedData, CompressedSize, IMAGE_MAGIC_BMP))
		{
			Format = EImageFormat::BMP;
		}
		else if (StartsWith((uint8*) CompressedData, CompressedSize, IMAGE_MAGIC_ICO))
		{
			Format = EImageFormat::ICO;
		}
		else if (StartsWith((uint8*) CompressedData, CompressedSize, IMAGE_MAGIC_EXR))
		{
			Format = EImageFormat::EXR;
		}
		else if (StartsWith((uint8*) CompressedData, CompressedSize, IMAGE_MAGIC_ICNS))
		{
			Format = EImageFormat::ICNS;
		}
		else if (StartsWith((uint8*)CompressedData, CompressedSize, IMAGE_MAGIC_HDR) ||
			StartsWith((uint8*)CompressedData, CompressedSize, IMAGE_MAGIC_HDR2))
		{
			Format = EImageFormat::HDR;
		}
		else if (StartsWith((uint8*)CompressedData, CompressedSize, IMAGE_MAGIC_TIFF_LITTLE_ENDIAN))
		{
			Format = EImageFormat::TIFF;
		}
		else if (StartsWith((uint8*)CompressedData, CompressedSize, IMAGE_MAGIC_TIFF_BIG_ENDIAN))
		{
			Format = EImageFormat::TIFF;
		}
		else if ( UE::DDS::FDDSFile::IsADDS((const uint8 *)CompressedData, CompressedSize) )
		{
			Format = EImageFormat::DDS;
		}
		else if ( FTgaImageWrapper::IsTGAHeader(CompressedData, CompressedSize) )
		{
			// beware: TGA does not have any type Id in the header
			//	this can mis-identify binaries as TGA
			//  it should be the last format checked in the list
			Format = EImageFormat::TGA;
		}

		return Format;
	}

	virtual const TCHAR * GetExtension(EImageFormat Format) override
	{
		switch(Format)
		{
		case EImageFormat::PNG: return TEXT("png");
		case EImageFormat::JPEG: return TEXT("jpg");
		case EImageFormat::GrayscaleJPEG: return TEXT("jpg");
		case EImageFormat::UEJPEG: return TEXT("uej");
		case EImageFormat::GrayscaleUEJPEG: return TEXT("uej");
		case EImageFormat::BMP: return TEXT("bmp");
		case EImageFormat::ICO: return TEXT("ico");
		case EImageFormat::EXR: return TEXT("exr");
		case EImageFormat::ICNS: return TEXT("icns");
		case EImageFormat::TGA: return TEXT("tga");
		case EImageFormat::HDR: return TEXT("hdr");
		case EImageFormat::TIFF: return TEXT("tiff");
		case EImageFormat::DDS: return TEXT("dds");
		case EImageFormat::Invalid:
		default:
			check(0);
			return nullptr;
		}
	}
	
	virtual EImageFormat GetImageFormatFromExtension(const TCHAR * Name) override
	{
		const TCHAR * Dot = FCString::Strrchr(Name,TEXT('.'));
		if ( Dot )
		{
			Name = Dot+1;
		}
		if ( FCString::Stricmp(Name,TEXT("png")) == 0 )
		{
			return EImageFormat::PNG;
		}
		else if ( FCString::Stricmp(Name,TEXT("jpg")) == 0
		 || FCString::Stricmp(Name,TEXT("jpeg")) == 0 )
		{
			return EImageFormat::JPEG;
		}
		else if (FCString::Stricmp(Name, TEXT("uej")) == 0)
		{
			return EImageFormat::UEJPEG;
		}
		else if ( FCString::Stricmp(Name,TEXT("bmp")) == 0 )
		{
			return EImageFormat::BMP;
		}
		else if ( FCString::Stricmp(Name,TEXT("ico")) == 0 )
		{
			return EImageFormat::ICO;
		}
		else if ( FCString::Stricmp(Name,TEXT("exr")) == 0 )
		{
			return EImageFormat::EXR;
		}
		else if ( FCString::Stricmp(Name,TEXT("icns")) == 0 )
		{
			return EImageFormat::ICNS;
		}
		else if ( FCString::Stricmp(Name,TEXT("tga")) == 0 )
		{
			return EImageFormat::TGA;
		}
		else if ( FCString::Stricmp(Name,TEXT("hdr")) == 0 )
		{
			return EImageFormat::HDR;
		}
		else if ( FCString::Stricmp(Name,TEXT("tiff")) == 0 ||
			 FCString::Stricmp(Name,TEXT("tif")) == 0 ||
			FCString::Stricmp(Name, TEXT("tx")) == 0)
		{
			return EImageFormat::TIFF;
		}
		else if ( FCString::Stricmp(Name,TEXT("dds")) == 0 )
		{
			return EImageFormat::DDS;
		}
		else
		{
			return EImageFormat::Invalid;
		}
	}
	
	virtual ERawImageFormat::Type ConvertRGBFormat(ERGBFormat RGBFormat,int BitDepth,bool * bIsExactMatch) override
	{
		return IImageWrapper::ConvertRGBFormat(RGBFormat,BitDepth,bIsExactMatch);
	}
	
	virtual void ConvertRawImageFormat(ERawImageFormat::Type RawFormat, ERGBFormat & OutFormat,int & OutBitDepth) override
	{
		IImageWrapper::ConvertRawImageFormat(RawFormat,OutFormat,OutBitDepth);
	}
	
	virtual EImageFormat GetDefaultOutputFormat(ERawImageFormat::Type RawFormat) override
	{
		switch(RawFormat)
		{
		case ERawImageFormat::G8:
		case ERawImageFormat::BGRA8:
		case ERawImageFormat::RGBA16:
		case ERawImageFormat::G16:
			return EImageFormat::PNG;

		case ERawImageFormat::BGRE8:
			return EImageFormat::HDR;

		case ERawImageFormat::RGBA16F:
		case ERawImageFormat::RGBA32F:
		case ERawImageFormat::R16F:
		case ERawImageFormat::R32F:
			return EImageFormat::EXR;
		
		default:
			check(0);
			return EImageFormat::Invalid;
		}
	}
	
	// Stress decoding an image many times with corruption
	//	to test if the ImageWrapper handles invalid data
	// this is by no means a substitute for real systematic fuzz testing, but is better than nothing
	void StressDecompressImage(TSharedPtr<IImageWrapper> ImageWrapper,const void* InCompressedData, int64 InCompressedSize)
	{
		FImage OutImage;

		// alloc a large aligned buf (would prefer to use VirtualAlloc directly so we get no-access pages adjacent)
		int64 AllocSize = (InCompressedSize + 0xFFFF) & (~0xFFFFULL);
		uint8 * Scratch = (uint8 *) FMemory::Malloc(AllocSize,65536);
		uint8 * End = Scratch + AllocSize;

		// test truncations of the stream :
		//for(int64 TruncatedSize = 1;TruncatedSize < InCompressedSize;TruncatedSize++)
		for(int64 TruncatedSize = InCompressedSize-1, Step=1; TruncatedSize>0; TruncatedSize -= Step, Step += Step+1)
		{
			// put truncated data at end of allocation to test reading past end :
			uint8 * TruncatedData = End - TruncatedSize;
			memcpy(TruncatedData,InCompressedData,TruncatedSize);
			
			if ( ImageWrapper->SetCompressed(TruncatedData,TruncatedSize) )
			{
				ImageWrapper->GetRawImage(OutImage);
			}
		}
		
		memcpy(Scratch,InCompressedData,InCompressedSize);

		// toggle one bit :
		for(int Bit=1; Bit < 256; Bit *= 2)
		{
			for(int64 Pos=0, Step=1;Pos < InCompressedSize;Pos+= Step, Step += Step+1)
			{
				Scratch[Pos] ^= Bit;
			
				if ( ImageWrapper->SetCompressed(Scratch,InCompressedSize) )
				{
					ImageWrapper->GetRawImage(OutImage);
				}

				Scratch[Pos] ^= Bit;
			}
		}

		FMemory::Free(Scratch);
	}

	virtual bool DecompressImage(const void* InCompressedData, int64 InCompressedSize, FImage & OutImage) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ImageWrapper.Decompress);

		OutImage = FImage();

		EImageFormat ImageFormat = DetectImageFormat(InCompressedData,InCompressedSize);
		if ( ImageFormat == EImageFormat::Invalid )
		{
			return false;
		}
		
		TSharedPtr<IImageWrapper> ImageWrapper = CreateImageWrapper(ImageFormat);
		if ( ! ImageWrapper.IsValid() )
		{
			return false;
		}
		
		#if 0
		// for testing :
		StressDecompressImage(ImageWrapper,InCompressedData,InCompressedSize);
		#endif

		if ( ! ImageWrapper->SetCompressed(InCompressedData,InCompressedSize) )
		{
			return false;
		}

		if ( ! ImageWrapper->GetRawImage(OutImage) )
		{
			return false;
		}

		return true;
	}

	virtual bool DecompressImage(const void* InCompressedData, int64 InCompressedSize, FDecompressedImageOutput& OutDecompressedImage) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ImageWrapper.Decompress);

		OutDecompressedImage = FDecompressedImageOutput();

		EImageFormat ImageFormat = DetectImageFormat(InCompressedData, InCompressedSize);
		if (ImageFormat == EImageFormat::Invalid)
		{
			return false;
		}

		TSharedPtr<IImageWrapper> ImageWrapper = CreateImageWrapper(ImageFormat);
		if (!ImageWrapper.IsValid())
		{
			return false;
		}

		if (!ImageWrapper->SetCompressed(InCompressedData, InCompressedSize))
		{
			return false;
		}

		if (!ImageWrapper->GetRawImage(OutDecompressedImage))
		{
			return false;
		}

		return true;
	}

	virtual bool CompressImage(TArray64<uint8> & OutData, EImageFormat ToFormat, const FImageView & InImage, int32 Quality) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ImageWrapper.Compress);

		OutData.Empty();

		TSharedPtr<IImageWrapper> ImageWrapper = CreateImageWrapper(ToFormat);
		if ( ! ImageWrapper.IsValid() )
		{
			return false;
		}

		ERGBFormat RGBFormat;
		int BitDepth;
		ConvertRawImageFormat(InImage.Format,RGBFormat,BitDepth);

		// can't save slices :
		check(InImage.NumSlices == 1 );

		FImageView WriteImage = InImage;
		FImage TempImage;

		if ( ! ImageWrapper->CanSetRawFormat(RGBFormat,BitDepth) )
		{
			// output image format may not support this pixel format
			//	in that case conversion is needed

			ERawImageFormat::Type NewFormat = ImageWrapper->GetSupportedRawFormat(InImage.Format);
			check( NewFormat != InImage.Format );
			
			// we will write to the image wrapper using "NewFormat"
			// do a blit to convert InImage to NewFormat

			TempImage.Init(InImage.SizeX,InImage.SizeY,NewFormat,ERawImageFormat::GetDefaultGammaSpace(NewFormat));
			WriteImage = TempImage;

			FImageCore::CopyImage(InImage,WriteImage);
			
			// re-get the RGBFormat we will set :
			ConvertRawImageFormat(NewFormat,RGBFormat,BitDepth);
			check( ImageWrapper->CanSetRawFormat(RGBFormat,BitDepth) );
		}
		else if ( InImage.GetGammaSpace() != ERawImageFormat::GetDefaultGammaSpace(InImage.Format) )
		{
			// ImageWrapper SetRaw() does not accept gamma information
			// it assumes U8 = SRGB and Float = Linear

			// eg. if you save a U8 surface with linear gamma
			// go ahead and write the U8 bytes without changing them
			// I think that's probably what is intended
			// when someone writes Linear U8 to PNG they don't want me to do a gamma transform
			
			// in some cases we could write the gamma setting to the output when possible, eg. for DDS output
			//  not doing that for now
		}

		if ( ! ImageWrapper->SetRaw(WriteImage.RawData,WriteImage.GetImageSizeBytes(),WriteImage.SizeX,WriteImage.SizeY,RGBFormat,BitDepth) )
		{
			return false;
		}

		OutData = ImageWrapper->GetCompressed(Quality);

		if ( OutData.IsEmpty() )
		{
			return false;
		}

		return true;
	}


public:

	//~ IModuleInterface interface

	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }
};


IMPLEMENT_MODULE(FImageWrapperModule, ImageWrapper);


#if PLATFORM_WINDOWS && !UE_BUILD_SHIPPING
extern "C" DLLEXPORT void png_write_sig2() {} // To work around png lib issue. Ask honk for details
#endif
