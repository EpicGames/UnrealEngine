// Copyright Epic Games, Inc. All Rights Reserved.

#include "TgaImageSupport.h"
#include "ImageWrapperPrivate.h"

namespace TgaImageSupportImpl
{
	
	static const uint8 * GetImageData(const FTGAFileHeader* TGA, const int64 TGABufferLength)
	{
		if ( TGABufferLength < sizeof(FTGAFileHeader) )
			return nullptr;

		int64 ColorMapEntryBytes = (TGA->ColorMapEntrySize + 7) / 8;
		int64 ColorMapBytes = ColorMapEntryBytes * TGA->ColorMapLength;
		
		int64 OffsetToImageData = sizeof(FTGAFileHeader) + TGA->IdFieldLength + ColorMapBytes;

		if ( TGABufferLength < OffsetToImageData )
			return nullptr;

		const uint8* ImageData = (const uint8*)TGA + OffsetToImageData;
		
		return ImageData;
	}
	
	static uint32 ConvertA1R5G5B5ToBGRA8(uint32 FilePixel)
	{
		// Convert file format A1R5G5B5 into pixel format B8G8R8A8
		uint32 TexturePixel;
		TexturePixel  = ((FilePixel & 0x001F) * 33) >> 2;
		TexturePixel += ((FilePixel & 0x03E0) * (33 << 1)) & 0x0000FF00;
		TexturePixel += ((FilePixel & 0x7C00) * (33 << 4)) & 0x00FF0000;
		TexturePixel += (FilePixel & 0x8000) * (255<<9);
		return TexturePixel;
	}
	
	static uint32 ConvertA1R5G5B5ToBGRA8(const uint8 * Bytes)
	{
		uint32 FilePixel = Bytes[0] + (Bytes[1]<<8);
		return ConvertA1R5G5B5ToBGRA8(FilePixel);
	}

	static uint32 ConvertBGRToBGRA8(const uint8 * BGR)
	{
		uint32 Pixel = BGR[0] + (BGR[1]<<8) + (BGR[2]<<16) + 0xFF000000U;
		return Pixel;
	}

	bool DecompressTGA_RLE_32bpp( const FTGAFileHeader* TGA, const int64 TGABufferLength, uint32* TextureData )
	{
		const uint8* ImageData = GetImageData(TGA,TGABufferLength);
		if ( ImageData == nullptr )
			return false;

		const uint8* TGAEnd = (uint8*)TGA + TGABufferLength;

		uint32 Pixel = 0;
		int32 RLERun = 0;
		int32 RAWRun = 0;

		for(int64 Y = TGA->Height-1; Y >=0; Y--) // Y-flipped.
		{
			for(int64 X = 0;X < TGA->Width;X++)
			{
				if( RLERun > 0 )
				{
					RLERun--;  // reuse current Pixel data.
				}
				else if( RAWRun == 0 ) // new raw pixel or RLE-run.
				{
					if ( TGAEnd <= ImageData )
					{
						return false;
					}
					uint8 RLEChunk = *(ImageData++);							
					if( RLEChunk & 0x80 )
					{
						RLERun = ( RLEChunk & 0x7F ) + 1;
						RAWRun = 1;
					}
					else
					{
						RAWRun = ( RLEChunk & 0x7F ) + 1;
					}

					if ( TGAEnd < ImageData + RAWRun * 4 )
					{
						return false;
					}
				}
				// Retrieve new pixel data - raw run or single pixel for RLE stretch.
				if( RAWRun > 0 )
				{
					Pixel = *(uint32*)ImageData; // RGBA 32-bit dword.
					ImageData += 4;
					RAWRun--;
					RLERun--;
				}
				// Store.
				*( (TextureData + Y*TGA->Width)+X ) = Pixel;
			}
		}

		return true;
	}

	bool DecompressTGA_RLE_24bpp( const FTGAFileHeader* TGA, const int64 TGABufferLength, uint32* TextureData )
	{
		const uint8* ImageData = GetImageData(TGA,TGABufferLength);
		if ( ImageData == nullptr )
			return false;

		const uint8* TGAEnd = (uint8*)TGA + TGABufferLength;

		uint32 Pixel = 0;
		int32 RLERun = 0;
		int32 RAWRun = 0;

		for(int64 Y = TGA->Height-1; Y >=0; Y--) // Y-flipped.
		{
			for(int64 X = 0;X < TGA->Width;X++)
			{
				if( RLERun > 0 )
				{
					RLERun--; // reuse current Pixel data.
				}
				else if( RAWRun == 0 ) // new raw pixel or RLE-run.
				{
					if ( TGAEnd <= ImageData )
					{
						return false;
					}
					uint8 RLEChunk = *(ImageData++);
					if( RLEChunk & 0x80 )
					{
						RLERun = ( RLEChunk & 0x7F ) + 1;
						RAWRun = 1;
					}
					else
					{
						RAWRun = ( RLEChunk & 0x7F ) + 1;
					}

					if ( TGAEnd < ImageData + RAWRun * 3 )
					{
						return false;
					}
				}
				// Retrieve new pixel data - raw run or single pixel for RLE stretch.
				if( RAWRun > 0 )
				{
					Pixel = ConvertBGRToBGRA8(ImageData);
					ImageData += 3;
					RAWRun--;
					RLERun--;
				}
				// Store.
				*( (TextureData + Y*TGA->Width)+X ) = Pixel;
			}
		}
		return true;
	}

	bool DecompressTGA_RLE_16bpp( const FTGAFileHeader* TGA, const int64 TGABufferLength, uint32* TextureData )
	{
		const uint8* ImageData = GetImageData(TGA,TGABufferLength);
		if ( ImageData == nullptr )
			return false;

		const uint8* TGAEnd = (uint8*)TGA + TGABufferLength;

		uint32 TexturePixel = 0;
		int32 RLERun = 0;
		int32 RAWRun = 0;

		for(int64 Y = TGA->Height-1; Y >=0; Y--) // Y-flipped.
		{					
			for( int64 X=0;X<TGA->Width;X++ )
			{						
				if( RLERun > 0 )
				{
					RLERun--;  // reuse current Pixel data.
				}
				else if( RAWRun == 0 ) // new raw pixel or RLE-run.
				{
					if ( TGAEnd <= ImageData )
					{
						return false;
					}
					uint8 RLEChunk =  *ImageData++;
					if( RLEChunk & 0x80 )
					{
						RLERun = ( RLEChunk & 0x7F ) + 1;
						RAWRun = 1;
					}
					else
					{
						RAWRun = ( RLEChunk & 0x7F ) + 1;
					}

					if ( TGAEnd < ImageData + RAWRun*2 )
					{
						return false;
					}
				}
				// Retrieve new pixel data - raw run or single pixel for RLE stretch.
				if( RAWRun > 0 )
				{ 
					TexturePixel = ConvertA1R5G5B5ToBGRA8(ImageData);
					ImageData += 2;
					RAWRun--;
					RLERun--;
				}
				// Store.
				*( (TextureData + Y*TGA->Width)+X ) = TexturePixel;
			}
		}

		return true;
	}

	bool DecompressTGA_32bpp( const FTGAFileHeader* TGA, const int64 TGABufferLength, uint32* TextureData )
	{
		const uint8* ImageData = GetImageData(TGA,TGABufferLength);
		if ( ImageData == nullptr )
			return false;

		if ( TGABufferLength < ((uint8*)ImageData - (uint8*)TGA) + (int64)TGA->Width * TGA->Height*4 )
		{
			return false;
		}

		for(int64 Y = 0;Y < TGA->Height;Y++)
		{
			FMemory::Memcpy(TextureData + Y * TGA->Width,ImageData + (TGA->Height - Y - 1) * TGA->Width * 4,TGA->Width * 4);
		}

		return true;
	}

	bool DecompressTGA_24bpp( const FTGAFileHeader* TGA, const int64 TGABufferLength, uint32* TextureData )
	{
		const uint8* ImageData = GetImageData(TGA,TGABufferLength);
		if ( ImageData == nullptr )
			return false;

		if ( TGABufferLength < ((uint8*)ImageData - (uint8*)TGA) + (int64)TGA->Width * TGA->Height * 3 )
		{
			return false;
		}

		for(int64 Y = 0; Y < TGA->Height; Y++)
		{
			for(int64 X = 0; X < TGA->Width; X++)
			{
				const uint8 * BGR = ImageData+( TGA->Height-Y-1 )*TGA->Width*3 +X*3;

				TextureData[Y*TGA->Width+X] = ConvertBGRToBGRA8(BGR);
			}
		}

		return true;
	}

	bool DecompressTGA_16bpp( const FTGAFileHeader* TGA, const int64 TGABufferLength, uint32* TextureData )
	{
		const uint8* ImageData = GetImageData(TGA,TGABufferLength);
		if ( ImageData == nullptr )
			return false;

		if ( TGABufferLength < ((uint8*)ImageData - (uint8*)TGA) + (int64)TGA->Height * TGA->Width * 2 )
		{
			return false;
		}

		for (int64 Y = TGA->Height - 1; Y >= 0; Y--)
		{					
			for (int64 X = 0; X < TGA->Width; X++)
			{
				*((TextureData + Y*TGA->Width) + X) = ConvertA1R5G5B5ToBGRA8(ImageData);
				ImageData += 2;					
			}
		}

		return true;
	}

	bool DecompressTGA_8bpp( const FTGAFileHeader* TGA, const int64 TGABufferLength, uint8* TextureData )
	{
		const uint8* ImageData = GetImageData(TGA,TGABufferLength);
		if ( ImageData == nullptr )
			return false;
			
		if ( TGABufferLength < ((uint8*)ImageData - (uint8*)TGA) + (int64)TGA->Width * TGA->Height )
		{
			return false;
		}

		int64 RevY = 0;
		for (int64 Y = TGA->Height-1; Y >= 0; --Y)
		{
			const uint8* ImageCol = ImageData + (Y * TGA->Width); 
			uint8* TextureCol = TextureData + (RevY++ * TGA->Width);
			FMemory::Memcpy(TextureCol, ImageCol, TGA->Width);
		}

		return true;
	}
}

bool DecompressTGA_helper( const FTGAFileHeader* TgaHeader, const int64 TGABufferLength, uint32* TextureData, const int64 TextureDataSize )
{
	bool bSuccess = false;
	if ( TgaHeader->ImageTypeCode == 10 ) // 10 = RLE compressed 
	{
		check( TextureDataSize == (int64) TgaHeader->Width * TgaHeader->Height * 4 );

		// RLE compression: CHUNKS: 1 -byte header, high bit 0 = raw, 1 = compressed
		// bits 0-6 are a 7-bit count; count+1 = number of raw pixels following, or rle pixels to be expanded. 
		if(TgaHeader->BitsPerPixel == 32)
		{
			bSuccess = TgaImageSupportImpl::DecompressTGA_RLE_32bpp(TgaHeader, TGABufferLength, TextureData);
		}
		else if( TgaHeader->BitsPerPixel == 24 )
		{
			bSuccess = TgaImageSupportImpl::DecompressTGA_RLE_24bpp(TgaHeader, TGABufferLength, TextureData);
		}
		else if( TgaHeader->BitsPerPixel == 16 )
		{
			bSuccess = TgaImageSupportImpl::DecompressTGA_RLE_16bpp(TgaHeader, TGABufferLength, TextureData);
		}
		else
		{
			UE_LOGF( LogImageWrapper, Error, "TgaHeader uses an unsupported rle-compressed bit-depth: %u", TgaHeader->BitsPerPixel );
			return false;
		}
	}
	else if(TgaHeader->ImageTypeCode == 2) // 2 = Uncompressed RGB
	{
		check( TextureDataSize == (int64)TgaHeader->Width * TgaHeader->Height * 4 );

		if(TgaHeader->BitsPerPixel == 32)
		{
			bSuccess = TgaImageSupportImpl::DecompressTGA_32bpp(TgaHeader, TGABufferLength, TextureData);
		}
		else if(TgaHeader->BitsPerPixel == 16)
		{
			bSuccess = TgaImageSupportImpl::DecompressTGA_16bpp(TgaHeader, TGABufferLength, TextureData);
		}
		else if(TgaHeader->BitsPerPixel == 24)
		{
			bSuccess = TgaImageSupportImpl::DecompressTGA_24bpp(TgaHeader, TGABufferLength, TextureData);
		}
		else
		{
			UE_LOGF( LogImageWrapper, Error, "TgaHeader uses an unsupported bit-depth: %u", TgaHeader->BitsPerPixel );
			return false;
		}
	}
	// Support for alpha stored as pseudo-color 8-bit TgaHeader
	else if(TgaHeader->ColorMapType == 1 && TgaHeader->ImageTypeCode == 1 && TgaHeader->BitsPerPixel == 8)
	{
		check( TextureDataSize == (int64) TgaHeader->Width * TgaHeader->Height * 1 );

		bSuccess = TgaImageSupportImpl::DecompressTGA_8bpp(TgaHeader, TGABufferLength, (uint8*)TextureData);
	}
	// standard grayscale
	else if(TgaHeader->ColorMapType == 0 && TgaHeader->ImageTypeCode == 3 && TgaHeader->BitsPerPixel == 8)
	{
		check( TextureDataSize == (int64) TgaHeader->Width * TgaHeader->Height * 1 );

		bSuccess = TgaImageSupportImpl::DecompressTGA_8bpp(TgaHeader, TGABufferLength, (uint8*)TextureData);
	}
	else
	{
		UE_LOGF( LogImageWrapper, Error, "TgaHeader is an unsupported type: %u", TgaHeader->ImageTypeCode );
		return false;
	}

	if (!bSuccess)
	{
		UE_LOGF(LogImageWrapper, Error, "The TGA file is invalid or corrupted");
		return false;
	}

	// Flip the image data if the flip bits are set in the TgaHeader header.
	const bool bFlipX = (TgaHeader->ImageDescriptor & 0x10) ? 1 : 0;
	const bool bFlipY = (TgaHeader->ImageDescriptor & 0x20) ? 1 : 0;
	if ( bFlipX || bFlipY )
	{
		TArray<uint8> FlippedData;
		FlippedData.AddUninitialized(TextureDataSize);

		int64 NumBlocksX = TgaHeader->Width;
		int64 NumBlocksY = TgaHeader->Height;
		int64 BlockBytes = TgaHeader->BitsPerPixel == 8 ? 1 : 4;
		
		check( TextureDataSize == NumBlocksX * NumBlocksY * BlockBytes );

		const uint8* MipData = (uint8*)TextureData;
		uint8 * FlippedDataPtr = &FlippedData[0];

		if ( BlockBytes == 1 )
		{
			for( int64 Y = 0; Y < NumBlocksY;Y++ )
			{
				for( int64 X  = 0; X < NumBlocksX; X++ )
				{
					int64 DestX = bFlipX ? (NumBlocksX - X - 1) : X;
					int64 DestY = bFlipY ? (NumBlocksY - Y - 1) : Y;
					
					FlippedDataPtr[DestX + DestY * NumBlocksX] = MipData[X + Y * NumBlocksX];
				}
			}
		}
		else
		{
			for( int64 Y = 0; Y < NumBlocksY;Y++ )
			{
				for( int64 X  = 0; X < NumBlocksX; X++ )
				{
					int64 DestX = bFlipX ? (NumBlocksX - X - 1) : X;
					int64 DestY = bFlipY ? (NumBlocksY - Y - 1) : Y;
					
					((uint32 *)FlippedDataPtr)[DestX + DestY * NumBlocksX] = ((uint32 *)MipData)[X + Y * NumBlocksX];
				}
			}
		}

		FMemory::Memcpy( TextureData, FlippedDataPtr, FlippedData.Num() );
	}

	return true;
}