// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FBitmapFileHeader;
struct FBitmapInfoHeader;

// Bitmap compression types.
enum EBitmapCompression
{
	BCBI_RGB            = 0,
	BCBI_RLE8           = 1,
	BCBI_RLE4           = 2,
	BCBI_BITFIELDS      = 3,
	BCBI_JPEG           = 4,
	BCBI_PNG            = 5,
	BCBI_ALPHABITFIELDS = 6,
};

// Bitmap info header versions.
enum class EBitmapHeaderVersion : uint8
{
	BHV_BITMAPINFOHEADER   = 0,
	BHV_BITMAPV2INFOHEADER = 1,
	BHV_BITMAPV3INFOHEADER = 2,
	BHV_BITMAPV4HEADER     = 3,
	BHV_BITMAPV5HEADER     = 4,
	BHV_INVALID = 0xFF
};

// Color space type of the bitmap, property introduced in Bitmap header version 4.
enum class EBitmapCSType : uint32
{
	BCST_BLCS_CALIBRATED_RGB     = 0x00000000,
	BCST_LCS_sRGB                = 0x73524742,
	BCST_LCS_WINDOWS_COLOR_SPACE = 0x57696E20,
	BCST_PROFILE_LINKED          = 0x4C494E4B,
	BCST_PROFILE_EMBEDDED        = 0x4D424544,
};

// .BMP file header.
#pragma pack(push,1)
struct FBitmapFileHeader
{
	uint16 bfType;
	uint32 bfSize;
	uint16 bfReserved1;
	uint16 bfReserved2;
	uint32 bfOffBits;

	friend FArchive& operator<<( FArchive& Ar, FBitmapFileHeader& H )
	{
		Ar << H.bfType << H.bfSize << H.bfReserved1 << H.bfReserved2 << H.bfOffBits;
		return Ar;
	}
};
#pragma pack(pop)

// .BMP subheader.
#pragma pack(push,1)
struct FBitmapInfoHeader
{
	uint32 biSize;
	uint32 biWidth;
	int32 biHeight;
	uint16 biPlanes;
	uint16 biBitCount;
	uint32 biCompression;
	uint32 biSizeImage;
	uint32 biXPelsPerMeter;
	uint32 biYPelsPerMeter;
	uint32 biClrUsed;
	uint32 biClrImportant;
	friend FArchive& operator<<( FArchive& Ar, FBitmapInfoHeader& H )
	{
		Ar << H.biSize << H.biWidth << H.biHeight;
		Ar << H.biPlanes << H.biBitCount;
		Ar << H.biCompression << H.biSizeImage;
		Ar << H.biXPelsPerMeter << H.biYPelsPerMeter;
		Ar << H.biClrUsed << H.biClrImportant;
		return Ar;
	}
	
public:
	EBitmapHeaderVersion GetHeaderVersion() const
	{
		// Since there is no field indicating the header version of the bitmap in the FileHeader,
		// the only way to know the format version is to check the header size.
		//
		// note that Adobe (incorrectly) writes biSize as 40 and then BMFH bfOffBits will be 52 or 56
		//	so this code does not detect those variant headers
		switch (biSize)
		{
			case 40:
				return EBitmapHeaderVersion::BHV_BITMAPINFOHEADER;
			case 52:
				// + RGB 32 bit masks
				return EBitmapHeaderVersion::BHV_BITMAPV2INFOHEADER;
			case 56:
				// + RGBA 32 bit masks
				return EBitmapHeaderVersion::BHV_BITMAPV3INFOHEADER;
			case 108:
				return EBitmapHeaderVersion::BHV_BITMAPV4HEADER;
			case 124:
				return EBitmapHeaderVersion::BHV_BITMAPV5HEADER;
			default:
				return EBitmapHeaderVersion::BHV_INVALID;
		}
	}
};
#pragma pack(pop)


// .BMP subheader V4
#pragma pack(push,1)
struct FBitmapInfoHeaderV4
{
	uint32 biSize;
	uint32 biWidth;
	int32 biHeight;
	uint16 biPlanes;
	uint16 biBitCount;
	uint32 biCompression;
	uint32 biSizeImage;
	uint32 biXPelsPerMeter;
	uint32 biYPelsPerMeter;
	uint32 biClrUsed;
	uint32 biClrImportant;
	uint32 biRedMask;
	uint32 biGreenMask;
	uint32 biBlueMask;
	uint32 biAlphaMask;
	uint32 biCSType;
	int32 biEndPointRedX;
	int32 biEndPointRedY;
	int32 bibiEndPointRedZ;
	int32 bibiEndPointGreenX;
	int32 biEndPointGreenY;
	int32 biEndPointGreenZ;
	int32 biEndPointBlueX;
	int32 biEndPointBlueY;
	int32 biEndPointBlueZ;
	uint32 biGammaRed;
	uint32 biGammaGreen;
	uint32 biGammaBlue;
	
	friend FArchive& operator<<(FArchive& Ar, FBitmapInfoHeaderV4& H)
	{
		Ar << H.biSize << H.biWidth << H.biHeight;
		Ar << H.biPlanes << H.biBitCount;
		Ar << H.biCompression << H.biSizeImage;
		Ar << H.biXPelsPerMeter << H.biYPelsPerMeter;
		Ar << H.biClrUsed << H.biClrImportant;
		Ar << H.biRedMask << H.biGreenMask << H.biBlueMask << H.biAlphaMask;
		Ar << H.biCSType << H.biEndPointRedX << H.biEndPointRedY << H.bibiEndPointRedZ;
		Ar << H.bibiEndPointGreenX << H.biEndPointGreenY << H.biEndPointGreenZ;
		Ar << H.biEndPointBlueX << H.biEndPointBlueY << H.biEndPointBlueZ;
		Ar << H.biGammaRed << H.biGammaGreen << H.biGammaBlue;
		return Ar;
	}
};
#pragma pack(pop)


#pragma pack(push, 1)
//Used by InfoHeaders pre-version 4, a structure that is declared after the FBitmapInfoHeader.
struct FBmiColorsMask
{
	// RGBA, in header pre-version 4, Alpha was only used as padding.
	uint32 RGBAMask[4];
};
#pragma pack(pop)