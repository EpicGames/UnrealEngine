// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "IImageWrapper.h"
#include "ImageCore.h"
#include "Internationalization/Text.h"

// http://radsite.lbl.gov/radiance/refer/Notes/picture_format.html
// http://paulbourke.net/dataformats/pic/

/** To load the HDR file image format. Does not support all possible types HDR formats (e.g. xyze is not supported) */
//  does not use ImageWrapperBase , unlike all other imagewrappers
class FHdrImageWrapper : public IImageWrapper
{
public:

	// Todo we should have this for all image wrapper.
	IMAGEWRAPPER_API bool SetCompressedFromView(TArrayView64<const uint8> Data);

	// IIMageWrapper Interface begin
	IMAGEWRAPPER_API virtual bool SetCompressed(const void* InCompressedData, int64 InCompressedSize) override;
	IMAGEWRAPPER_API virtual bool SetRaw(const void* InRawData, int64 InRawSize, const int32 InWidth, const int32 InHeight, const ERGBFormat InFormat, const int32 InBitDepth, const int32 InBytesPerRow) override;
	IMAGEWRAPPER_API virtual TArray64<uint8> GetCompressed(int32 Quality = 0) override;
	IMAGEWRAPPER_API virtual bool GetRaw(const ERGBFormat InFormat, int32 InBitDepth, TArray64<uint8>& OutRawData) override;
	IMAGEWRAPPER_API virtual bool GetRaw(const ERGBFormat InFormat, int32 InBitDepth, FDecompressedImageOutput& OutDecompressedImage) override;
	
	IMAGEWRAPPER_API virtual bool CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const override;
	IMAGEWRAPPER_API virtual ERawImageFormat::Type GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const override;

	IMAGEWRAPPER_API virtual int64 GetWidth() const override;
	IMAGEWRAPPER_API virtual int64 GetHeight() const override;
	IMAGEWRAPPER_API virtual int32 GetBitDepth() const override;
	IMAGEWRAPPER_API virtual ERGBFormat GetFormat() const override;

	// IImageWrapper Interface end

	// GetErrorMessage : nice idea, but not virtual, never called by the standard import path
	IMAGEWRAPPER_API const FText& GetErrorMessage() const;

	IMAGEWRAPPER_API void FreeCompressedData();

	/**
	 * Does this image type support embedded metadata in its header?
	 *
	 * PNG is an example of an image type which supports adding user-defined metadata to its header.
	 */
	virtual bool SupportsMetadata() const override;

	/**
	 * Adds a key and value to this image's metadata.  Will be saved in the image's header and restored when the image is loaded.
	 *
	 * @param InKey Metadata consists of key value pairs.
	 * @param InValue Metadata consists of key value pairs.
	 */
	virtual void AddMetadata(const FString& InKey, const FString& InValue) override;

	/**
	 * Copies this image's metadata to the caller's map.
	 *
	 * @param OutMetadata A copy of this image's metadata.
	 */
	virtual void GetMetadata(TStringMap& OutMetadata) const override;

	using IImageWrapper::GetRaw;

private:
	// Helpers for error exits. Set error messages and return false.
	void SetAndLogError(const FText& InText);
	bool FailHeaderParsing(); // also calls FreeCompressData.
	bool FailUnexpectedEOB();
	bool FailMalformedScanline();

	bool GetHeaderLine(const uint8*& BufferPos, char Line[256]);

	static bool ParseMatchString(const char*& InOutCursor, const char* InExpected);
	static bool ParsePositiveInt(const char*& InOutCursor, int* OutValue);
	static bool ParseImageSize(const char* InLine, int* OutWidth, int* OutHeight);

	static bool HaveBytes(const uint8* InCursor, const uint8* InEnd, int InAmount);

	/** @param Out order in bytes: RGBE */
	bool DecompressScanline(uint8* Out, const uint8*& In, const uint8* InEnd);

	bool OldDecompressScanline(uint8* Out, const uint8*& In, const uint8* InEnd, int32 Length, bool bInitialRunAllowed);

	bool IsCompressedImageValid() const;

	TArrayView64<const uint8> CompressedData;
	const uint8* RGBDataStart = nullptr;

	TArray64<uint8> CompressedDataHolder;
	TArray64<uint8> RawDataHolder;

	/** INDEX_NONE if not valid */
	int64 Width = INDEX_NONE;
	/** INDEX_NONE if not valid */
	int64 Height = INDEX_NONE;

	// Reported error
	FText ErrorMessage;
};
