// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "ImageCore.h"
#include "ImageWrapperOutputTypes.h"

/**

NOTE: you should not write code that talks directly to individual ImageWrappers

Instead use ImageWrapperModule CompressImage/DecompressImage

Prefer the new interface that go through FImage not TArray of bytes


**/

/**
 * Enumerates the types of image formats this class can handle.
 */
enum class EImageFormat : int8
{
	/** Invalid or unrecognized format. */
	Invalid = -1,

	/** Portable Network Graphics. */
	PNG = 0,

	/** Joint Photographic Experts Group. */
	JPEG,

	/** Single channel JPEG. */
	GrayscaleJPEG,	

	/** Windows Bitmap. */
	BMP,

	/** Windows Icon resource. */
	ICO,

	/** OpenEXR (HDR) image file format. */
	EXR,

	/** Mac icon. */
	ICNS,
	
	/** Truevision TGA / TARGA */
	TGA,

	/** Hdr file from radiance using RGBE */
	HDR,

	/** Tag Image File Format files */
	TIFF,

	/** DirectDraw Surface */
	DDS,

	/** UE JPEG format. */
	UEJPEG,

	/** Single channel UE JPEG. */
	GrayscaleUEJPEG,
};


/**
 * Enumerates the types of RGB formats this class can handle.
 */
enum class ERGBFormat : int8
{
	Invalid = -1,

	// Red, Green, Blue and Alpha ; requires RB swap from FColor
	RGBA =  0,

	// Blue, Green, Red and Alpha ; is ERawImageFormat::BGRA8 and Fcolor
	BGRA =  1,

	// Gray scale
	Gray =  2,

	// Red, Green, Blue and Alpha using IEEE Floating-Point Arithmetic (see IEEE754). The format is always binary.
	RGBAF = 3,

	// Blue, Green, Red and Exponent (Similar to the RGBE format from radiance but with the blue and red channel inversed)
	BGRE =  4,

	// Gray scale using IEEE Floating-Point Arithmetic (see IEEE754). The format is always binary.
	GrayF = 5,
};


/**
 * Enumerates available image compression qualities.
 * 
 * JPEG interprets Quality as 1-100
 * JPEG default quality is 85 , Uncompressed means 100
 * 
 * for PNG:
 * Negative qualities in [-1,-9] set PNG zlib level
 * PNG interprets "Uncompressed" as zlib level 0 (none)
 * otherwise default zlib level 3 is used.
 * 
 * EXR respects the "Uncompressed" flag to turn off compression; otherwise ZIP_COMPRESSION is used.
 */
enum class EImageCompressionQuality : int8
{
	Default = 0,
	Uncompressed = 1,
	Max = 100,
};


/**
 * Interface for image wrappers.
 *
 * to Encode:
 *	SetRaw() then GetCompressed()
 * to Decode :
 *  SetCompressed() then GetRaw()
 *
 * in general, direct use of the IImageWrapper interface is now discouraged
 * use ImageWrapperModule CompressImage/DecompressImage instead.
 */
class IImageWrapper
{
protected:
	// debug image name string for any errors or warnings
	const TCHAR* DebugImageName =  nullptr;

public:


	/**  
	 * Sets the compressed data.  Can then call GetRaw().
	 *
	 * @param InCompressedData The memory address of the start of the compressed data.
	 * @param InCompressedSize The size of the compressed data parsed.
	 * @return true if data was the expected format.
	 * 
	 * after SetCompressed, image info queries like GetWidth and GetBitDepth are allowed
	 * call GetRaw to get the decoded bits
	 * decompression is not done until GetRaw
	 */
	virtual bool SetCompressed(const void* InCompressedData, int64 InCompressedSize) = 0;

	/**  
	 * Sets the raw image data.  Prepares to call GetCompressed() next.
	 *
	 * @param InRawData The memory address of the start of the raw data.
	 * @param InRawSize The size of the compressed data parsed.
	 * @param InWidth The width of the image data.
	 * @param InHeight the height of the image data.
	 * @param InFormat the format the raw data is in, normally RGBA.
	 * @param InBitDepth the bit-depth per channel, normally 8.
	 * @param InBytesPerRow the number of bytes between rows, 0 = tightly packed rows with no padding.
	 * @return true if data was the expected format.
	 * 
	 * you must not SetRaw() with a format unless it passes CanSetRawFormat()
	 * deprecated : avoid direct calls to SetRaw(), use ImageWrapperModule CompressImage instead
	 * do not use InBytesPerRow, it is ignored
	 * SetRaw does not take gamma information
	 *  assumes U8 = SRGB and all else = Linear
	 */
	virtual bool SetRaw(const void* InRawData, int64 InRawSize, const int32 InWidth, const int32 InHeight, const ERGBFormat InFormat, const int32 InBitDepth, const int32 InBytesPerRow = 0) = 0;

	/** CanSetRawFormat returns true if SetRaw will accept this format */
	virtual bool CanSetRawFormat(const ERGBFormat InFormat, const int32 InBitDepth) const = 0;

	/** 
	 * returns InFormat if supported, else maps to something supported
	 * the returned format will pass CanSetRawFormat()
	 */
	virtual ERawImageFormat::Type GetSupportedRawFormat(const ERawImageFormat::Type InFormat) const = 0;


	/**
	 * Gets the compressed data.  (call SetRaw first)
	 * (Note: It may consume the data set in the SetCompressed function if it was set before)
	 * 
	 * @return Array of the compressed data.  returns empty array on failure
	 * 
	 */
	virtual TArray64<uint8> GetCompressed(int32 Quality = 0) = 0;

	/**
	 * Gets the data for export. 
	 * Usually the same thing as GetCompressed.
	 * 
	 * @return Array of the data to export.  returns empty array on failure
	 * 
	 */
	virtual TArray64<uint8> GetExportData(int32 Quality = 0)
	{
		return GetCompressed(Quality);
	}

	/**
	 * GetRaw after SetCompressed
	 * fills the raw data in the native format/depth contained in the file
	 * Do not use the GetRaw() variants that take format/depth arguments.
	 * 
	 * @param OutRawData    Filled with raw image data.
	 *
	 * This GetRaw call replaces the variants with format/depth arguments, but prefer GetRawImage instead.
	 */
	bool GetRaw(TArray64<uint8>& OutRawData)
	{
		ERGBFormat Format = GetFormat();
		int32 BitDepth = GetBitDepth();

		// Format and BitDepth should have been set by SetCompressed
		if ( Format == ERGBFormat::Invalid || BitDepth == 0 )
		{
			return false;
		}

		return GetRaw(Format,BitDepth,OutRawData);
	}

	/**
	 * GetRaw after SetCompressed
	 * fills the raw data in the native format/depth contained in the file along with meta info
	 * Do not use the GetRaw() variants that take format/depth arguments.
	 *
	 * @param OutRawData				Filled with raw image data.
	 * @param OutDecompressedImage		Filled with mip images and other metadata.
	 *
	 * This GetRaw call replaces the variants with format/depth arguments, but prefer GetRawImage instead.
	 */
	bool GetRaw(FDecompressedImageOutput& OutDecompressedImage)
	{
		ERGBFormat Format = GetFormat();
		int32 BitDepth = GetBitDepth();

		// Format and BitDepth should have been set by SetCompressed
		if (Format == ERGBFormat::Invalid || BitDepth == 0)
		{
			return false;
		}

		return GetRaw(Format, BitDepth, OutDecompressedImage);
	}
	
	/**
	 * Decode the image file data from SetCompressed() into an FImage
	 * OutImage is allocated and attributes are filled
	 * Any previous passed-in contents of OutImage are destroyed
	 * OutImage.Format is ignored, a new format is set from the loaded image
	 *
	 * @param OutImage	 Filled with the image
	 *
	 * This is the recommended API to get the raw image data from the imagewrapper.
	 * Prefer this instead of any of the GetRaw() calls.
	 */
	bool GetRawImage(FImage & OutImage);

	/**
	 * Decode the image file data from SetCompressed() into an FImage
	 * OutImage is allocated and attributes are filled.
	 * Any previous passed-in contents of OutImage are destroyed.
	 * OutImage.Format is ignored, a new format is set from the loaded image
	 * MetaInfo is filled if applicable
	 * 
	 * @param OutDecompressedImage	Image and meta data along with Mip Map Images
	 * 
	 * This is the recommended API to get the raw image data from the imagewrapper.
	 * Prefer this instead of any of the GetRaw() calls.
	 */
	bool GetRawImage(FDecompressedImageOutput& OutDecompressedImage);

	
	/**  
	 * Gets the raw data.
	 * (Note: It may consume the data set in the SetRaw function if it was set before)
	 *
	 * @param InFormat How we want to manipulate the RGB data.
	 * @param InBitDepth The output bit-depth per channel, normally 8.
	 * @param OutRawData Will contain the uncompressed raw data.
	 * @return true on success, false otherwise.
	 *
	 * this is often broken, should only be used with InFormat == GetFormat()
	 * DEPRECATED , use GetRaw() with 1 argument or GetRawImage()
	 */
	virtual bool GetRaw(const ERGBFormat InFormat, int32 InBitDepth, TArray64<uint8>& OutRawData) = 0;

	/**
	 * Gets the raw data and meta info
	 * (Note: It may consume the data set in the SetRaw function if it was set before)
	 *
	 * @param InFormat How we want to manipulate the RGB data.
	 * @param InBitDepth The output bit-depth per channel, normally 8.
	 * @param OutRawData Will contain the uncompressed raw data.
	 * @param OutDecompressedImage Will contain the mip images if available and other metadata.
	 * @return true on success, false otherwise.
	 *
	 * this is often broken, should only be used with InFormat == GetFormat()
	 * DEPRECATED , use GetRaw() with 2 argument or GetRawImage() meta info overload
	 */
	virtual bool GetRaw(const ERGBFormat InFormat, int32 InBitDepth, FDecompressedImageOutput& OutDecompressedImage) = 0;
		
	/**
	 * Gets the raw data in a TArray. Only use this if you're certain that the image is less than 2 GB in size.
	 * Prefer using the overload which takes a TArray64 in general.
	 * (Note: It may consume the data set in the SetRaw function if it was set before)
	 *
	 * @param InFormat How we want to manipulate the RGB data.
	 * @param InBitDepth The output bit-depth per channel, normally 8.
	 * @param OutRawData Will contain the uncompressed raw data.
	 * @return true on success, false otherwise.
	 *
	 * this is often broken, should only be used with InFormat == GetFormat()
	 * DEPRECATED , use GetRaw() with 1 argument or GetRawImage()
	 */
	bool GetRaw(const ERGBFormat InFormat, int32 InBitDepth, TArray<uint8>& OutRawData)
	{
		TArray64<uint8> TmpRawData;
		if (GetRaw(InFormat, InBitDepth, TmpRawData) && ensureMsgf(TmpRawData.Num() == (int32)TmpRawData.Num(), TEXT("Tried to get %" INT64_FMT "x%" INT64_FMT " %dbpp image with format %d into 32-bit TArray (%lld bytes)"), GetWidth(), GetHeight(), InBitDepth, EnumToUnderlyingType(InFormat), (long long int)TmpRawData.Num()))
		{
			OutRawData = MoveTemp(TmpRawData);
			return true;
		}
		else
		{
			return false;
		}
	}
	
	/**
	 * Get the raw version of the image and write to the array view
	 * (Note: It may consume the data set in the SetRaw function if it was set before)
	 *
	 * @param InFormat How we want to manipulate the RGB data.
	 * @param InBitDepth The output bit-depth per channel, normally 8.
	 * @param OutRawData Will contain the uncompressed raw data.
	 * @return true on success, false otherwise.
	 *
	 * this is often broken, should only be used with InFormat == GetFormat()
	 * DEPRECATED , use GetRaw() with 1 argument or GetRawImage()
	 */
	bool GetRaw(const ERGBFormat InFormat, int32 InBitDepth, TArrayView64<uint8> OutRawData)
	{
		TArray64<uint8> TmpRawData;
		if (GetRaw(InFormat, InBitDepth, TmpRawData))
		{
			if (ensureMsgf(TmpRawData.Num() == OutRawData.Num(), TEXT("The view doesn't have the proper size to receive the texture.")))
			{
				FPlatformMemory::Memcpy(OutRawData.GetData(), TmpRawData.GetData(), OutRawData.Num());
				return true;
			}
		}

		return false;
	}

	/**
	 * Gets the width of the image.
	 *
	 * @return Image width.
	 * @see GetHeight
	 */
	virtual int64 GetWidth() const = 0;

	/** 
	 * Gets the height of the image.
	 *
	 * @return Image height.
	 * @see GetWidth
	 */
	virtual int64 GetHeight() const = 0;

	/** 
	 * Gets the bit depth of the image.
	 *
	 * @return The bit depth per-channel of the image.
	 *
	 * Beware several of the old wrappers (BMP,TGA) incorrectly used to return bits per *color* not per channel
	 * they now correctly return per-channel.
	 */
	virtual int32 GetBitDepth() const = 0;

	/** 
	 * Gets the format of the image.
	 * Theoretically, this is the format it would be best to call GetRaw() with, if you support it.
	 *
	 * @return The format the image data is in
	 */
	virtual ERGBFormat GetFormat() const = 0;
	
	
	/* Should the pixels be treated as sRGB encoded? (or Linear)
	* 
	* note: ImageWrapper Format does not track if pixels are Gamma/SRGB or not
	* assume they are ERawImageFormat::GetDefaultGammaSpace gammaspace
	* eg. U8 is SRGB and everything else is Linear
	*/
	bool GetSRGB() const
	{
		// sRGB is guessed from bit depth
		// 8 = on (except BGRE)
		// must match ERawImageFormat::GetDefaultGammaSpace
		return GetBitDepth() == 8 && GetFormat() != ERGBFormat::BGRE;
	}

	// external users call these from ImageWrapperModule.h , see documentation there
	IMAGEWRAPPER_API static void ConvertRawImageFormat(ERawImageFormat::Type RawFormat, ERGBFormat & OutFormat,int & OutBitDepth);
	IMAGEWRAPPER_API static ERawImageFormat::Type ConvertRGBFormat(ERGBFormat RGBFormat,int BitDepth,bool * bIsExactMatch = nullptr);
	
	IMAGEWRAPPER_API static int64 GetRGBFormatBytesPerPel(ERGBFormat RGBFormat,int BitDepth);

	/* get the current image format, mapped to an ERawImageFormat
	 * if ! *bIsExactMatch , conversion is needed
	 * can call after SetCompressed()
	 */
	ERawImageFormat::Type GetClosestRawImageFormat(bool * bIsExactMatch = nullptr) const
	{
		ERGBFormat Format = GetFormat();
		int BitDepth = GetBitDepth();
		ERawImageFormat::Type Ret = ConvertRGBFormat(Format,BitDepth,bIsExactMatch);
		return Ret;
	}

	/* Set the debug image name
	 */
	void SetDebugImageName(const TCHAR* InDebugImageName)
	{
		DebugImageName = InDebugImageName;
	}

	/**
	 * Does this image type support embedded metadata in its header?
	 *
	 * PNG is an example of an image type which supports adding user-defined metadata to its header.
	 */
	virtual bool SupportsMetadata() const = 0;

	/**
	 * Adds a key and value to this image's metadata.  Will be saved in the image's header and restored when the image is loaded.
	 *
	 * @param InKey Metadata consists of key value pairs.
	 * @param InValue Metadata consists of key value pairs.
	 */
	virtual void AddMetadata(const FString& InKey, const FString& InValue) = 0;

	using TStringMap = TMap<FString, FString>;
	/**
	 * Copies this image's metadata to the caller's map.
	 *
	 * @param OutMetadata A copy of this image's metadata.
	 */
	virtual void GetMetadata(TStringMap& OutMetadata) const = 0;

public:

	/** Virtual destructor. */
	virtual ~IImageWrapper() = default;
};
