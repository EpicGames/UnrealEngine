// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "ImageCore.h"
#include "IImageWrapper.h"
#include "ImageWrapperOutputTypes.h"




/**
 * Interface for image wrapper modules.
 *
 * DO NOT WRITE CODE THAT USES IMAPE WRAPPERS DIRECTLY.
 * 
 * If you have "Engine" module, then just use FImageUtils::CompressImage/DecompressImage
 * The easiest way to load/save an image is FImageUtils::LoadImage/SaveImageByExtension
 *
 * If you don't have "Engine" module, then use IImageWrapperModule::CompressImage/DecompressImage
 * Do not manually create individual image wrappers.
 *
 * NOTE: Please prefer to work with images through ImageCore FImage/FImageView 
 * (not TextureSource/TextureSourceFormat or raw arrays of bytes)
 *  again, do not manually use ERGBFormat and arrays of bytes to work with ImageWrappers
 *  instead use the FImage-oriented APIs which will handle all format conversions for you.
 * 
 * You do NOT need to convert formats before saving an image, the FImage APIs will do it for you.
 *
 * If you got an image from a render target, it may have a zero alpha channel.  You might want to do
 *   FImageCore::SetAlphaOpaque before saving.
 *
 * 
 * note on SRGB/Gamma handling :
 * it is assumed that non-U8 data is always Linear
 * U8 data is written without changing the bytes
 * the gamma correction of U8 bytes is NOT persisted in the file formats (for read or write)
 * loading will always give you SRGB for U8 (ERawImageFormat::GetDefaultGammaSpace)
 * if U8-float conversions are required, they do respect gamma space
 * 
 * eg. if you write U8 Linear data to EXR, it will be converted to Linear float from U8 Linear correctly
 * if you write U8 Linear to BMP , it will write the U8 bytes unchanged, and on load it will come back in as U8 SRGB
 * 
 */
class IImageWrapperModule
	: public IModuleInterface
{
public:


	/**  
	 * Convert input FImage into a file-format encoded array.
 	 * in ImageWrapper land, "Compress" means "put in file format"
	 * OutData is filled with the file format encoded data
	 * lossy conversion of the pixel type may be done if necessary
	 * eg. if you pass F32 float pixels to write to BMP, they will be converted to SRGB U8 BGRA8
	 * 
	 * @param OutData		Filled with image-format data
	 * @param ToFormat		Image format to encode to
	 * @param InImage		Image to encode
	 * @param Quality		50-100 for JPEG, or EImageCompressionQuality
	 * 
	**/
	virtual bool CompressImage(TArray64<uint8> & OutData, EImageFormat ToFormat, const FImageView & InImage, int32 Quality = 0) = 0;
	
	/* Read an image from file format encoded data.
	 * ImageWrapper calls this a "decompress"
	 * OutImage is allocated and filled, any existing contents are discarded
	 * 
	 * @param InCompressedData	Image format encoded bytes to read; format is automatically deduced from the content
	 * @param InCompressedSize	Size of InCompressedData in bytes
	 * @param OutImage			Filled with Image that is read.  Allocated.
	 */
	virtual bool DecompressImage(const void* InCompressedData, int64 InCompressedSize, FImage & OutImage) = 0;

	/* Read an image from file format encoded data.
	 * ImageWrapper calls this a "decompress"
	 * OutImage is allocated and filled, any existing contents are discarded
	 *
	 * @param InCompressedData					Image format encoded bytes to read; format is automatically deduced from the content
	 * @param InCompressedSize					Size of InCompressedData in bytes
	 * @param OutDecompressedImageData			Filled with Image along with other meta data and mip images that is read.  Allocated.
	 */
	virtual bool DecompressImage(const void* InCompressedData, int64 InCompressedSize, FDecompressedImageOutput& OutDecompressedImage) = 0;

	/**  
	 * Create an IImageWrapper helper of a specific type 
	 *
	 * @param InFormat - The type of image we want to deal with
	 * @param InOptionalDebugImageName - An optional string to be displayed with any errors or warnings
	 * 
	 * @return The helper base class to manage the data
	 * EImageFormat is a compressor / file format, not a pixel format
	 * Deprecated.  Prefer CompressImage/DecompressImage.
	 */
	virtual TSharedPtr<IImageWrapper> CreateImageWrapper(const EImageFormat InFormat, const TCHAR* InOptionalDebugImageName = nullptr) = 0;

	/**
	 * Detect image format by looking at the first few bytes of the compressed image data.
	 * You can call this method as soon as you have 8-16 bytes of compressed file content available.
	 *
	 * @param InCompressedData The raw image header.
	 * @param InCompressedSize The size of InCompressedData.
	 * @return the detected format or EImageFormat::Invalid if the method could not detect the image format.
	 */
	virtual EImageFormat DetectImageFormat(const void* InCompressedData, int64 InCompressedSize) = 0;

	/* Name can be a full name like "xx.png" or just the extension part, like "png"
	 * returns EImageFormat::Invalid if no supported image extension found
	 */
	virtual EImageFormat GetImageFormatFromExtension(const TCHAR * Name) = 0;

	/* returns extension, not including the "." , 3 or 4 chars */
	virtual const TCHAR * GetExtension(EImageFormat Format) = 0;

	/* get a good default output image format for a pixel format */
	virtual EImageFormat GetDefaultOutputFormat(ERawImageFormat::Type RawFormat) = 0;

	/* Convert an ImageWrapper style {ERGBFormat+BitDepth} into an ERawImageFormat for FImage
	* returns ERawImageFormat::Invalid if no mapping is possible
	* bIsExactMatch is filled with whether the formats are an exact match or not
	*	if not, conversion is needed
	*/
	virtual ERawImageFormat::Type ConvertRGBFormat(ERGBFormat Format,int BitDepth,bool * bIsExactMatch = nullptr) = 0;
	
	/**
	* Convert an FImage ERawImageFormat into an ImageWrapper style {ERGBFormat+BitDepth} 
	* mapping is always possible and requires no conversion
	*/
	virtual void ConvertRawImageFormat(ERawImageFormat::Type RawFormat, ERGBFormat & OutFormat,int & OutBitDepth) = 0;

public:

	/** Virtual destructor. */
	virtual ~IImageWrapperModule() = default;
};
