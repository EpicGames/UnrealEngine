// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/StringFwd.h"
#include "CoreMinimal.h"
#include "IImageWrapper.h"

// ImageWrapperHelper is deprecated, use IImageWrapperModule instead
class ImageWrapperHelper
{
public:

	// deprecated, use IImageWrapperModule::GetExtension() instead
	static IMAGEWRAPPER_API FStringView GetFormatExtension(EImageFormat InImageFormat, bool bIncludeDot=false);

	// deprecated, use IImageWrapperModule::GetImageFormatFromExtension instead
	static IMAGEWRAPPER_API EImageFormat GetImageFormat(FStringView StringExtention);

	static IMAGEWRAPPER_API const FStringView GetImageFilesFilterString(bool bIncludeAllFiles);
};
