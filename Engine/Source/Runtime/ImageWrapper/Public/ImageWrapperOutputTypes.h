// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreTypes.h"
#include "ImageCore.h"

/**
 * Use this struct to get mip images and other metadata about the image.
 */
struct FDecompressedImageOutput
{
	FMipMapImage MipMapImage;

	/** Meta data */
	FString ApplicationVendor;
	FString ApplicationName;
	FString ApplicationVersion;
};