// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Serialization/Archive.h"

#pragma pack(push,1)
struct FTGAFileHeader
{
	uint8 IdFieldLength;
	uint8 ColorMapType;
	uint8 ImageTypeCode;		// 2 for uncompressed RGB format
	uint16 ColorMapOrigin;
	uint16 ColorMapLength;
	uint8 ColorMapEntrySize;
	uint16 XOrigin;
	uint16 YOrigin;
	uint16 Width;
	uint16 Height;
	uint8 BitsPerPixel;
	uint8 ImageDescriptor;

	friend FArchive& operator<<( FArchive& Ar, FTGAFileHeader& H )
	{
		Ar << H.IdFieldLength << H.ColorMapType << H.ImageTypeCode;
		Ar << H.ColorMapOrigin << H.ColorMapLength << H.ColorMapEntrySize;
		Ar << H.XOrigin << H.YOrigin << H.Width << H.Height << H.BitsPerPixel;
		Ar << H.ImageDescriptor;
		return Ar;
	}
};
#pragma pack(pop)

/**
 * This helper allows to decompress TGA data in a pre-allocated memory block.
 * The pixel format is necessarily PF_A8R8G8B8.
 */
IMAGEWRAPPER_API bool DecompressTGA_helper(	const FTGAFileHeader* TGA, const int64 TGABufferLength, uint32* TextureData, const int64 TextureDataSize );
